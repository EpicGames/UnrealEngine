{
    "FileVersion" : 3,
    "Version" : 1,
    "VersionName" : "0.1",
    "FriendlyName" : "DataflowVolumetric",
    "Description" : "Adds volumetric support to Dataflow",
    "Category": "Dataflow",
    "CreatedBy" : "Epic Games, Inc.",
    "CreatedByURL" : "https://epicgames.com",
    "DocsURL" : "",
    "MarketplaceURL" : "",
    "SupportURL" : "",
    "CanContainContent" : true,
    "IsBetaVersion" : false,
    "Installed" : false,
	"Modules": [
		{
			"Name": "DataflowVolumeCore",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Linux", "Mac"
			]
		},
		{
			"Name": "DataflowVolumeNodes",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Linux", "Mac"
			]
		}
	],
	"Plugins": [
		{
			"Name": "Dataflow",
			"Enabled": true
		}
	]
}
