// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataflowVolumeCore : ModuleRules
	{
        public DataflowVolumeCore(ReadOnlyTargetRules Target) : base(Target)
		{
			// For boost:: and TBB:: code
			bUseRTTI = true;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] 
				{
					"DynamicMesh",
					"MeshConversion",
					"Core",
					"CoreUObject",
					"ChaosCore",
					"Chaos",
					"DataflowCore",
					"DataflowEngine",
					"DataflowEnginePlugin",
					"GeometryCollectionEngine",
					"GeometryCore",
					"Voronoi",
					"GeometryFramework",
					"MeshDescription",
					"StaticMeshDescription",
					"PlanarCut",
					"Engine",
					"DataflowEditor"
				}
			);

			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"IntelTBB",
				"OpenVDB"
			);
		}
	}
}
