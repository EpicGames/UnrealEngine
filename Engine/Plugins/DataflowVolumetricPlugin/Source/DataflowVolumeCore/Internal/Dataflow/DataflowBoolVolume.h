// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "DataflowVolume.h"

#include "DataflowBoolVolume.generated.h"

namespace UE::DataflowVolume::Private
{
	class FDataflowBoolVolumeImpl;
}

/* --------------------------------------------------------------------------------------------------------------- */
/* FDataflowBoolVolume */
/* --------------------------------------------------------------------------------------------------------------- */

USTRUCT()
struct FDataflowBoolVolume : public FDataflowVolume
{
	inline static const FName TypeName = "BoolVolume";

	GENERATED_USTRUCT_BODY()

public:
	FDataflowBoolVolume() {}
	virtual ~FDataflowBoolVolume() {}

	TSharedPtr<const UE::DataflowVolume::Private::FDataflowBoolVolumeImpl> GetBoolVolumeImpl() const;
	TSharedPtr<UE::DataflowVolume::Private::FDataflowBoolVolumeImpl> GetBoolVolumeImpl();

	DATAFLOWVOLUMECORE_API FVector GetVoxelSize() const;

private:
	FDataflowBoolVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowBoolVolumeImpl> InBoolVolume);
};


