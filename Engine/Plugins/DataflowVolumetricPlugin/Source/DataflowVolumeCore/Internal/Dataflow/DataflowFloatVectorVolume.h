// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowVolume.h"
#include "Dataflow/DataflowVolumeNodeEnums.h"

#include "DataflowFloatVectorVolume.generated.h"

namespace UE::DataflowVolume::Private
{
	class FDataflowFloatVectorVolumeImpl;
}

struct FDataflowFloatVolume;
struct FDataflowBoolVolume;

/* --------------------------------------------------------------------------------------------------------------- */
/* FDataflowFloatVectorVolume */
/* --------------------------------------------------------------------------------------------------------------- */

USTRUCT()
struct FDataflowFloatVectorVolume : public FDataflowVolume
{
	inline static const FName TypeName = "FloatVectorVolume";

	GENERATED_USTRUCT_BODY()

public:
	friend struct FDataflowFloatVolume;

	FDataflowFloatVectorVolume() {}
	virtual ~FDataflowFloatVectorVolume() {}

	TSharedPtr<const UE::DataflowVolume::Private::FDataflowFloatVectorVolumeImpl> GetFloatVectorVolumeImpl() const;
	TSharedPtr<UE::DataflowVolume::Private::FDataflowFloatVectorVolumeImpl> GetFloatVectorVolumeImpl();

	DATAFLOWVOLUMECORE_API FVector GetVoxelSize() const;
	DATAFLOWVOLUMECORE_API FString GetVectorType() const;

	DATAFLOWVOLUMECORE_API void VolumeSample(const TArray<FVector>& InPoints, TArray<FVector>& OutValues) const;

	DATAFLOWVOLUMECORE_API FDataflowFloatVolume ComputeDivergence(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume = nullptr) const;
	DATAFLOWVOLUMECORE_API FDataflowFloatVolume ComputeMagnitude(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume = nullptr) const;
	DATAFLOWVOLUMECORE_API FDataflowFloatVectorVolume ComputeCurl(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume = nullptr) const;
	DATAFLOWVOLUMECORE_API FDataflowFloatVectorVolume ComputeNormalize(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume = nullptr) const;

private:
	FDataflowFloatVectorVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowFloatVectorVolumeImpl> InFloatVectorVolume);
};

