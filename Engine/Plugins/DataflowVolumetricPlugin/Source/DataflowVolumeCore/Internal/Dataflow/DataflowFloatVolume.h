// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowVolume.h"
#include "DataflowVolumeNodeEnums.h"

#include "DataflowFloatVolume.generated.h"

namespace UE::DataflowVolume::Private
{
	class FDataflowFloatVolumeImpl;
}

struct FDataflowIntVolume;
struct FDataflowBoolVolume;
struct FDataflowFloatVectorVolume;
struct FMeshDescription;

/* --------------------------------------------------------------------------------------------------------------- */
/* FDataflowFloatVolume */
/* --------------------------------------------------------------------------------------------------------------- */
USTRUCT()
struct FDataflowFloatVolume : public FDataflowVolume
{
	inline static const FName TypeName = "FloatVolume";

	GENERATED_USTRUCT_BODY()

public:
	FDataflowFloatVolume() {}
	virtual ~FDataflowFloatVolume() {}

	TSharedPtr<const UE::DataflowVolume::Private::FDataflowFloatVolumeImpl> GetFloatVolumeImpl() const;
	TSharedPtr<UE::DataflowVolume::Private::FDataflowFloatVolumeImpl> GetFloatVolumeImpl();

	DATAFLOWVOLUMECORE_API FVector GetVoxelSize() const;

	// Create a sphere SDF
	DATAFLOWVOLUMECORE_API static FDataflowFloatVolume CreateSphereSDF(const float InVoxelSize, const float InRadius, const FVector& InCenter);

	/** Create a cube SDF */
	DATAFLOWVOLUMECORE_API static FDataflowFloatVolume CreateCubeSDF(const float InScale, const FVector& InCenter, const float InVoxelSize);

	/** Create different types of solid SDF */
	DATAFLOWVOLUMECORE_API static FDataflowFloatVolume CreatePlatonicSolidSDF(const int32 InFaceCount, const float InScale, const FVector& InCenter, const float InVoxelSize);

	/** Create a volume from MeshDescription */
	DATAFLOWVOLUMECORE_API static FDataflowFloatVolume CreateVolumeFromMeshDescription(
		const FMeshDescription& InMeshDescription,
		const FTransform& InTransform,
		const float InVoxelSize,
		const FDataflowVolume& InReferenceGrid,
		const EDataflowVolumeOutputType InOutputType,
		const FString& InGridName,
		const bool InUseWorldSpaceUnits,
		const float InExteriorBand,
		const float InInteriorBand,
		const int32 InExteriorBandVoxels,
		const int32 InInteriorBandVoxels,
		const bool bFillInterior,
		const bool bPreserveHoles,
		const float InIsoValue,
		FDataflowIntVolume& OutFaceIndexVolume);

	/** Compute sphere packing in an SDF volume using original algorithm */
	DATAFLOWVOLUMECORE_API void VolumeToSpheres(
		const int32 InMinSphereCount,
		const int32 InMaxSphereCount,
		const bool InOverlapping,
		const float InMinRadius,
		const float InMaxRadius,
		const float InIsoValue,
		const int32 InInstanceCount,
		TArray<FVector>& OutSphereCenters,
		TArray<float>& OutSphereRadii) const;

	/** Compute sphere packing in an SDF volume using improved algorithm */
	DATAFLOWVOLUMECORE_API void VolumeToSpheresImproved(
		const int32 InMinSphereCount,
		const int32 InMaxSphereCount,
		const bool InOverlapping,
		const float InMinRadius,
		const float InMaxRadius,
		const float InIsoValue,
		const int32 InInstanceCount,
		const int32 InRandomSeed,
		const EDataflowVolumeScatterType InScatterType,
		const float InSpread,
		const float InMinNumberOfPointsPerVoxel,
		const float InMaxNumberOfPointsPerVoxel,
		TArray<FVector>& OutSphereCenters,
		TArray<float>& OutSphereRadii) const;

	/** Convert SDF to Fog Volume */
	DATAFLOWVOLUMECORE_API FDataflowFloatVolume CovertSDFToFogVolume(const bool InPruneTolerance, const float InTolerance, const bool InFloodFillOutput, const bool InActivateInteriror) const;

	/** Convert SDF to Fog Volume */
	DATAFLOWVOLUMECORE_API FDataflowFloatVolume CovertFogVolumeToSDF(
		const float InFogIsoValue,
		const bool InPruneTolerance,
		const float InTolerance,
		const bool InFloodFillOutput,
		const bool InActivateInteriror) const;

	/** Convert volume to MeshDescription */
	DATAFLOWVOLUMECORE_API void ConvertVolumeToMeshDescription(const float InIsoValue, const float InAdaptivity, FMeshDescription& OutMeshDescription) const;

	/** Compute uniform point scatter in an SDF volume */
	DATAFLOWVOLUMECORE_API void UniformVolumeScatter(
		const int32 InMinNumberOfPoints,
		const int32 InMaxNumberOfPoints,
		const int32 InRandomSeed,
		const float InIsoValue,
		const float InSpread,
		TArray<FVector>& OutPoints) const;

	/** Compute dense uniform point scatter in an SDF volume */
	DATAFLOWVOLUMECORE_API void DenseUniformVolumeScatter(
		const float InMinNumberOfPointsPerVoxel,
		const float InMaxNumberOfPointsPerVoxel,
		const int32 InRandomSeed,
		const float InIsoValue,
		const float InSpread,
		TArray<FVector>& OutPoints) const;

	/** Compute a non uniform point scatter in an SDF volume */
	DATAFLOWVOLUMECORE_API void NonUniformVolumeScatter(
		const float InMinNumberOfPointsPerVoxel,
		const float InMaxNumberOfPointsPerVoxel,
		const int32 InRandomSeed,
		const float InIsoValue,
		const float InSpread,
		TArray<FVector>& OutPoints) const;

	/** Sample a volume */
	DATAFLOWVOLUMECORE_API void VolumeSample(const TArray<FVector>& InPoints, TArray<float>& OutValues) const;

	/** Compute gradient */
	DATAFLOWVOLUMECORE_API FDataflowFloatVectorVolume ComputeGradient(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume = nullptr) const;

	/** Compute curvature */
	DATAFLOWVOLUMECORE_API FDataflowFloatVolume ComputeCurvature(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume = nullptr) const;

	/** Compute Laplacian */
	DATAFLOWVOLUMECORE_API FDataflowFloatVolume ComputeLaplacian(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume = nullptr) const;

	/** Compute ClosestPoint */
	DATAFLOWVOLUMECORE_API FDataflowFloatVectorVolume ComputeClosestPoint(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume = nullptr) const;

	/** Combine SDF volumes */
	DATAFLOWVOLUMECORE_API FDataflowFloatVolume VolumeCombine(
		const FDataflowFloatVolume& InVolumeB,
		const EDataflowVolumeSDFCombineOperation InOperation,
		const float InMultiplierA,
		const float InMultiplierB,
		const EDataflowVolumeSDFCombineResample InResample,
		const EDataflowVolumeSDFCombineInterpolation InInterpolation,
		const bool InPruneDegenerateTiles,
		const float InPruneTol) const;

	/** Compute sphere packing in an SDF volume with relaxation */
	DATAFLOWVOLUMECORE_API void VolumeToSpheresWithRelaxation(
		const int32 InPointCount,
		const int32 InScatterRandomSeed,
		const float InMinRadius,
		const float InMaxRadius,
		const int32 InRandomSeed,
		const float InIsoValue,
		const float InSpread,
		const bool InRemoveSpheres,
		const bool InApplyRelaxation,
		const int32 InSteps,
		const float InStepScalar,
		const bool InCorrectAgainstSurface,
		const float InDistanceThreshold,
		TArray<FSphere>& OutSpheres) const;

private:
	friend struct FDataflowFloatVectorVolume;
	friend class UE::DataflowVolume::Private::FDataflowFloatVolumeImpl;

	FDataflowFloatVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowFloatVolumeImpl> InFloatVolume);
};

