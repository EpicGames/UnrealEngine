// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "DataflowVolume.h"

#include "DataflowIntVolume.generated.h"

namespace UE::DataflowVolume::Private
{
	class FDataflowIntVolumeImpl;
	class FDataflowFloatVolumeImpl;
}

/* --------------------------------------------------------------------------------------------------------------- */
/* FDataflowIntVolume */
/* --------------------------------------------------------------------------------------------------------------- */
USTRUCT()
struct FDataflowIntVolume : public FDataflowVolume
{
	inline static const FName TypeName = "IntVolume";

	GENERATED_USTRUCT_BODY()

public:
	FDataflowIntVolume() {}
	virtual ~FDataflowIntVolume() {}

	TSharedPtr<const UE::DataflowVolume::Private::FDataflowIntVolumeImpl> GetIntVolumeImpl() const;
	TSharedPtr<UE::DataflowVolume::Private::FDataflowIntVolumeImpl> GetIntVolumeImpl();

	DATAFLOWVOLUMECORE_API FVector GetVoxelSize() const;

	/** Sample a volume */
	DATAFLOWVOLUMECORE_API void VolumeSample(const TArray<FVector>& InPoints, TArray<int32>& OutValues) const;

private:
	friend struct FDataflowFloatVolume;

	FDataflowIntVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowIntVolumeImpl> InIntVolume);

	void SetIntVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowIntVolumeImpl> InIntVolume);
};

