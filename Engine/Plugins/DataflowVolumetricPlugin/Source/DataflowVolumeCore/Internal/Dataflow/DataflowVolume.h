// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "DataflowVolume.generated.h"

namespace UE::DataflowVolume::Private
{
	class FDataflowVolumeImpl;
}

class UVolumeTexture;

/* --------------------------------------------------------------------------------------------------------------- */
/* FDataflowVolume */
/* --------------------------------------------------------------------------------------------------------------- */
USTRUCT()
struct FDataflowVolume
{
	GENERATED_USTRUCT_BODY()

public:
	FDataflowVolume() {}
	virtual ~FDataflowVolume() {}

	DATAFLOWVOLUMECORE_API int32 GetNumActiveVoxels(const float InIsovalue, bool bInteriorMaskOnly = false) const;
	DATAFLOWVOLUMECORE_API void GetActiveVoxels(const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly = false) const;
	DATAFLOWVOLUMECORE_API FBox GetVolumeBoundingBox() const;

	DATAFLOWVOLUMECORE_API bool CreateVolumeTexture(UVolumeTexture* InVolumeTexture) const;

	/** Prints Volume info into string */
	DATAFLOWVOLUMECORE_API FString VolumeInfo() const;
	FName GetType() const { return Type; }

	/*
	
	 How to use this:
	 
	 GetValue() for AnyType returns the storage type
	 
	 @code
	 const FDataflowVolume& Volume = GetValue(Context, &MyVolume);
	
	 if (const FDataflowFloatVolume* FloatVolume = Volume.Cast<FDataflowFloatVolume>())
	 {
	    ...
	 }	
	 else if (const FDataflowIntVolume* IntVolume = Volume.Cast<FDataflowIntVolume>())
	 {
	    ...
	 }
	 else if (const FDataflowFloatVectorVolume* FloatVectorVolume = Volume.Cast<FDataflowFloatVectorVolume>())
	 {
	    ...
	 }
	 @endcode

	*/

	template<typename T>
	T* Cast()
	{
		if constexpr (std::is_base_of_v<FDataflowVolume, T>)
		{
			if (Type == T::TypeName)
			{
				return static_cast<T*>(this);
			}
		}

		return nullptr;
	}

	template<typename T>
	const T* Cast() const
	{
		if constexpr (std::is_base_of_v<FDataflowVolume, T>)
		{
			if (Type == T::TypeName)
			{
				return static_cast<const T*>(this);
			}
		}

		return nullptr;
	}

	TSharedPtr<const UE::DataflowVolume::Private::FDataflowVolumeImpl> GetVolume() const { return Volume; }

protected:
	FDataflowVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowVolumeImpl> InVolume, const FName InType);

	TSharedPtr<UE::DataflowVolume::Private::FDataflowVolumeImpl> Volume;

private:
	FName Type;
};

