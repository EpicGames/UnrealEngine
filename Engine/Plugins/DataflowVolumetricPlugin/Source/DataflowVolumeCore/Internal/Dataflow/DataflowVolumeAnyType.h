// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowAnyType.h"
#include "Dataflow/DataflowTypePolicy.h"
#include "Dataflow/DataflowVolume.h"
#include "Dataflow/DataflowFloatVolume.h"
#include "Dataflow/DataflowIntVolume.h"
#include "Dataflow/DataflowFloatVectorVolume.h"

#include "DataflowVolumeAnyType.generated.h"

UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowVolume)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowFloatVolume)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowIntVolume)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowFloatVectorVolume)

struct FDataflowVolumeTypePolicy :
	public TDataflowMultiTypePolicy<FDataflowFloatVolume, FDataflowIntVolume, FDataflowFloatVectorVolume>
{
};

USTRUCT()
struct FDataflowVolumeTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowVolumeTypePolicy;
	using FStorageType = FDataflowVolume;

	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FDataflowVolume Value;
};

template <>
struct FDataflowConverter<FDataflowVolume>
{
	template <typename TFromType>
	static void From(const TFromType& From, FDataflowVolume& To)
	{
		To = From; // from typed to general FDataflowVolume
	}

	template <typename TToType>
	static void To(const FDataflowVolume& From, TToType& To)
	{
		if (const TToType& Typed = From.Cast<TToType>())
		{
			To = Typed;
		}
	}
};

namespace UE::Dataflow
{
	DATAFLOWVOLUMECORE_API void RegisterVolumeAnyTypes();
}

