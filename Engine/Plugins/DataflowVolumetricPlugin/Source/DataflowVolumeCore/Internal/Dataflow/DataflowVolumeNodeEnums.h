// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "DataflowVolumeNodeEnums.generated.h"

UENUM(BlueprintType)
enum class EDataflowVolumeOutputType : uint8
{
	SDF UMETA(DisplayName = "Signed Distance Field"),
	USDF UMETA(DisplayName = "Unsigned Distance Field"),
	FogVolume UMETA(DisplayName = "Fog Volume"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeSDFCombineOperation : uint8
{
	CopyA UMETA(DisplayName = "Copy A"),
	CopyB UMETA(DisplayName = "Copy B"),
	InvertA UMETA(DisplayName = "Invert A"),
	Add UMETA(DisplayName = "Add"),
	Subtract UMETA(DisplayName = "Subtract"),
	Multiply UMETA(DisplayName = "Multiply"),
	Divide UMETA(DisplayName = "Divide"),
	Minimum UMETA(DisplayName = "Minimum"),
	Maximum UMETA(DisplayName = "Maximum"),
	CompATimesB UMETA(DisplayName = "(1 - A) * B"),
	APlusCompATimesB UMETA(DisplayName = "A + (1 - A) * B"),
	SDFUnion UMETA(DisplayName = "SDF Union"),
	SDFIntersect UMETA(DisplayName = "SDF Intersection"),
	SDFDifference UMETA(DisplayName = "SDF Difference"),
	ReplaceWithActive UMETA(DisplayName = "Replace A with Active B"),
	ActivityUnion UMETA(DisplayName = "Activity Union"),
	ActivityIntersect UMETA(DisplayName = "Activity Intersection"),
	ActivityDifference UMETA(DisplayName = "Activity Difference")
};

UENUM(BlueprintType)
enum class EDataflowVolumeSDFCombineResample : uint8
{
	Off UMETA(DisplayName = "Off"),
	BMatchA UMETA(DisplayName = "B to Match A"),
	AMatchB UMETA(DisplayName = "A to Match B"),
	HiresMatchLores UMETA(DisplayName = "Higher-res to Match Lower-res"),
	LoresMatchHires UMETA(DisplayName = "Lower-res to Match Higher-res")
};

UENUM(BlueprintType)
enum class EDataflowVolumeSDFCombineInterpolation : uint8
{
	Nearest UMETA(DisplayName = "Nearest"),
	Linear UMETA(DisplayName = "Linear"),
	Quadratic UMETA(DisplayName = "Quadratic")
};

UENUM(BlueprintType)
enum class EDataflowVolumeConvertSDFTo : uint8
{
	Volume UMETA(DisplayName = "Volume"),
	Collection UMETA(DisplayName = "Collection"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeConvertSDFGridClass : uint8
{
	NoChange UMETA(DisplayName = "No Change"),
	SDFToFog UMETA(DisplayName = "SDF -> Fog Volume"),
	FogToSDF UMETA(DisplayName = "Fog Volume -> SDF"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeConvertSDFGridType : uint8
{
	NoChange UMETA(DisplayName = "No Change"),
	Float UMETA(DisplayName = "Float"),
	Int UMETA(DisplayName = "Int"),
	Bool UMETA(DisplayName = "Bool"),
	VectorFloat UMETA(DisplayName = "Vector Float"),
	VectorInt UMETA(DisplayName = "Vector Int"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeActiveVoxelsDisplayType : uint8
{
	Points UMETA(DisplayName = "Points"),
	WwireframeBoxes UMETA(DisplayName = "Wireframe Boxes"),
	SolidBoxes UMETA(DisplayName = "Solid Boxes"),
};

UENUM(BlueprintType)
enum class EDataflowVolumePlatonicSolidType : uint8
{
	Tetrahedron UMETA(DisplayName = "Tetrahedron"),
	Cube UMETA(DisplayName = "Cube"),
	Octahedron UMETA(DisplayName = "Octahedron"),
	Dodecahedron UMETA(DisplayName = "Dodecahedron"),
	Icosahedron UMETA(DisplayName = "Icosahedron"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeSampleAttribute : uint8
{
	Phi UMETA(DisplayName = "Vertex/Phi"),
	Gradient UMETA(DisplayName = "Vertex/Gradient"),
	VertexColor UMETA(DisplayName = "Vertex/Color"),
	VertexClosestFaceID UMETA(DisplayName = "Vertex/ClosestFaceID"),
	VertexUV UMETA(DisplayName = "Vertex/UV"),
	FaceMaterialID UMETA(DisplayName = "Face/MaterialID"),
	VertexCustomAttribute UMETA(DisplayName = "Vertex/Custom Attribute"),
	FaceCustomAttribute UMETA(DisplayName = "Face/Custom Attribute"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeSampleType : uint8
{
	Float UMETA(DisplayName = "Float"),
	Int UMETA(DisplayName = "Int"),
	FloatVector UMETA(DisplayName = "FloatVector"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeSampleLocation : uint8
{
	Vertices UMETA(DisplayName = "Vertices"),
	FaceCenters UMETA(DisplayName = "Face Centers"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeSliceMethod : uint8
{
	Points UMETA(DisplayName = "Points"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeSlicePlane : uint8
{
	XYPlane UMETA(DisplayName = "XY Plane"),
	YZPlane UMETA(DisplayName = "YZ Plane"),
	ZXPlane UMETA(DisplayName = "ZX Plane"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeSliceRamp : uint8
{
	InfraRed UMETA(DisplayName = "Infra-Red"),
	WhiteToRed UMETA(DisplayName = "White to Red"),
	Grayscale UMETA(DisplayName = "Grayscale"),
	BlackBody UMETA(DisplayName = "Blackbody"),
	Custom UMETA(DisplayName = "Custom"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeAnalysisOperator : uint8
{
	Gradient UMETA(DisplayName = "Gradient (Scalar->Vector)"),
	Curvature UMETA(DisplayName = "Curvature (Scalar->Scalar)"),
	Laplacian UMETA(DisplayName = "Laplacian (Scalar->Scalar)"),
	ClosestPoint UMETA(DisplayName = "Closest Point (Scalar->Vector)"),
	Divergence UMETA(DisplayName = "Divergence (Vector->Scalar)"),
	Curl UMETA(DisplayName = "Curl (Vector->Vector)"),
	Magnitude UMETA(DisplayName = "Magnitude (Vector->Scalar)"),
	Normalize UMETA(DisplayName = "Normalize (Vector->Vector)"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeAnalysisOutputName : uint8
{
	AppendOperationName UMETA(DisplayName = "Append Operation Name"),
	CustomName UMETA(DisplayName = "Custom Name"),
};

UENUM(BlueprintType)
enum class EDataflowVolumeScatterType : uint8
{
	Uniform UMETA(DisplayName = "Uniform"),
	DenseUniform UMETA(DisplayName = "Dense Uniform"),
	NonUniform UMETA(DisplayName = "Non Uniform")
};


