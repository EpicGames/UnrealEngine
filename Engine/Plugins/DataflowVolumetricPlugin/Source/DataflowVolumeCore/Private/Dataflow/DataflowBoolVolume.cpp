// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowBoolVolume.h"
#include "Dataflow/DataflowBoolVolumeImpl.h"

/* --------------------------------------------------------------------------------------------------------------- */
/* FDataflowBoolVolume Implementation */
/* --------------------------------------------------------------------------------------------------------------- */

FDataflowBoolVolume::FDataflowBoolVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowBoolVolumeImpl> InBoolVolume)
	: FDataflowVolume(InBoolVolume, FDataflowBoolVolume::TypeName)
{
	static_assert(sizeof(FDataflowVolume) == sizeof(*this));
}

/* --------------------------------------------------------------------------------------------------------------- */

TSharedPtr<const UE::DataflowVolume::Private::FDataflowBoolVolumeImpl> FDataflowBoolVolume::GetBoolVolumeImpl() const
{
	return StaticCastSharedPtr<const UE::DataflowVolume::Private::FDataflowBoolVolumeImpl>(Volume);
}

/* --------------------------------------------------------------------------------------------------------------- */

TSharedPtr<UE::DataflowVolume::Private::FDataflowBoolVolumeImpl> FDataflowBoolVolume::GetBoolVolumeImpl()
{
	return StaticCastSharedPtr<UE::DataflowVolume::Private::FDataflowBoolVolumeImpl>(Volume);
}

/* --------------------------------------------------------------------------------------------------------------- */

FVector FDataflowBoolVolume::GetVoxelSize() const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowBoolVolumeImpl> BoolVolumeImpl = GetBoolVolumeImpl())
	{
		return BoolVolumeImpl->GetVoxelSize();
	}

	return {};
}

