// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowBoolVolumeImpl.h"
#include <Chaos/ArrayND.h>
#include "Engine/VolumeTexture.h"
#include "Dataflow/DataflowVolumeUtils.h"

namespace UE::DataflowVolume::Private
{
	/* --------------------------------------------------------------------------------------------------------------- */
	/* FDataflowBoolVolumeImpl */
	/* --------------------------------------------------------------------------------------------------------------- */
	void FDataflowBoolVolumeImpl::SetGrid(const openvdb::BoolGrid::Ptr& InBoolGrid, const FString& InGridName)
	{
		FDataflowVolumeImpl::SetGrid(InBoolGrid, InGridName);

		BoolGrid = InBoolGrid;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	openvdb::BoolGrid::Ptr FDataflowBoolVolumeImpl::GetGrid() const
	{
		return BoolGrid;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FString FDataflowBoolVolumeImpl::GetGridType() const
	{
		if (BoolGrid)
		{
			std::string GridType = BoolGrid->gridType();
			return FString(GridType.c_str());
		}

		return {};
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FString FDataflowBoolVolumeImpl::VolumeInfo() const
	{
		FString BaseString = FDataflowVolumeImpl::VolumeInfo();

		FString OutputStr;
		OutputStr.Appendf(TEXT("\n----------------------------------------\n"));
		OutputStr += BaseString;

		FString GridType = this->GetGridType();

		OutputStr.Appendf(TEXT("Type: %s\n"),
			*GridType);

		OutputStr.Appendf(TEXT("----------------------------------------\n"));

		return OutputStr;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowBoolVolumeImpl::GetActiveVoxels(const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly) const
	{
		if (BoolGrid)
		{
			UE::DataflowVolumeUtils::GetActiveVoxels<openvdb::BoolGrid>(BoolGrid, InIsovalue, OutActiveVoxels, bInteriorMaskOnly);
		}
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FBox FDataflowBoolVolumeImpl::GetVolumeBoundingBox() const
	{
		return this->GetActiveVoxelBoundingBox();
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	bool FDataflowBoolVolumeImpl::CreateVolumeTexture(UVolumeTexture* InVolumeTexture) const
	{
		if (InVolumeTexture == nullptr)
		{
			return false;
		}

		const EPixelFormat PixelFormat = EPixelFormat::PF_R16F;

		if (BoolGrid && !BoolGrid->empty())
		{
			openvdb::Coord ActiveVoxelDim = BoolGrid->evalActiveVoxelDim();
			openvdb::CoordBBox Bbox = BoolGrid->evalActiveVoxelBoundingBox();
			const int32 XMin = Bbox.min().x();
			const int32 YMin = Bbox.min().y();
			const int32 ZMin = Bbox.min().z();

			openvdb::BoolGrid::Accessor Accessor = BoolGrid->getAccessor();
			
			auto QueryVoxel = [&Accessor, &XMin, &YMin, &ZMin](int32 PosX, int32 PosY, int32 PosZ, void* Value)
			{
				openvdb::Coord Coord(PosX + XMin, PosY + YMin, PosZ + ZMin);
				FFloat16* const Voxel = static_cast<FFloat16*>(Value);

				if (Accessor.isValueOn(Coord))
				{
					Voxel[0].Set((float)Accessor.getValue(Coord));
				}
				else
				{
					Voxel[0].Set(0.f);
				}
			};

			const bool bSuccess = InVolumeTexture->UpdateSourceFromFunction(QueryVoxel, ActiveVoxelDim[0], ActiveVoxelDim[1], ActiveVoxelDim[2], TSF_R16F);

			if (!bSuccess)
			{
				return false;
			}

			return true;
		}

		return false;
	}
}

