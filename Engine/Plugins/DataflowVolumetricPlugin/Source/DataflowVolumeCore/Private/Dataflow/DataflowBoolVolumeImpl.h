// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowVolumeImpl.h"
#include "Dataflow/OpenVDB.h"

namespace UE::DataflowVolume::Private
{
	/* --------------------------------------------------------------------------------------------------------------- */
	/* FDataflowBoolVolumeImpl */
	/* --------------------------------------------------------------------------------------------------------------- */
	class FDataflowBoolVolumeImpl : public FDataflowVolumeImpl
	{
	private:
		openvdb::BoolGrid::Ptr BoolGrid = nullptr;

	public:
		// Virtual functions
		virtual void GetActiveVoxels(const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly = false) const override;
		virtual FBox GetVolumeBoundingBox() const override;
		virtual bool CreateVolumeTexture(UVolumeTexture* InVolumeTexture) const override;

		openvdb::BoolGrid::Ptr GetGrid() const;
		void SetGrid(const openvdb::BoolGrid::Ptr& InBoolGrid, const FString& InGridName = "");
		FString GetGridType() const;
		FString VolumeInfo() const;

	};
}



