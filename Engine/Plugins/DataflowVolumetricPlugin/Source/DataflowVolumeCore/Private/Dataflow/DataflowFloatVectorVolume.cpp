// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowFloatVectorVolume.h"
#include "Dataflow/DataflowFloatVolume.h"
#include "Dataflow/DataflowBoolVolume.h"
#include "Dataflow/DataflowFloatVolumeImpl.h"
#include "Dataflow/DataflowBoolVolumeImpl.h"
#include "Dataflow/DataflowFloatVectorVolumeImpl.h"

/* --------------------------------------------------------------------------------------------------------------- */
/* FDataflowFloatVectorVolume Implementation */
/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVectorVolume::FDataflowFloatVectorVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowFloatVectorVolumeImpl> InFloatVectorVolume)
	: FDataflowVolume(InFloatVectorVolume, FDataflowFloatVectorVolume::TypeName)
{
	static_assert(sizeof(FDataflowVolume) == sizeof(*this));
}

/* --------------------------------------------------------------------------------------------------------------- */

TSharedPtr<const UE::DataflowVolume::Private::FDataflowFloatVectorVolumeImpl> FDataflowFloatVectorVolume::GetFloatVectorVolumeImpl() const
{
	return StaticCastSharedPtr<const UE::DataflowVolume::Private::FDataflowFloatVectorVolumeImpl>(Volume);
}

/* --------------------------------------------------------------------------------------------------------------- */

TSharedPtr<UE::DataflowVolume::Private::FDataflowFloatVectorVolumeImpl> FDataflowFloatVectorVolume::GetFloatVectorVolumeImpl()
{
	return StaticCastSharedPtr<UE::DataflowVolume::Private::FDataflowFloatVectorVolumeImpl>(Volume);
}

/* --------------------------------------------------------------------------------------------------------------- */

FString FDataflowFloatVectorVolume::GetVectorType() const
{
	return GetFloatVectorVolumeImpl()->GetVectorType();
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowFloatVectorVolume::VolumeSample(const TArray<FVector>& InPoints, TArray<FVector>& OutValues) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVectorVolumeImpl> FloatVectorVolumeImpl = GetFloatVectorVolumeImpl())
	{
		FloatVectorVolumeImpl->VolumeSample(InPoints, OutValues);
	}
}

/* --------------------------------------------------------------------------------------------------------------- */

FVector FDataflowFloatVectorVolume::GetVoxelSize() const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVectorVolumeImpl> FloatVectorVolumeImpl = GetFloatVectorVolumeImpl())
	{
		return FloatVectorVolumeImpl->GetVoxelSize();
	}

	return {};
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume FDataflowFloatVectorVolume::ComputeDivergence(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVectorVolumeImpl> FloatVectorVolumeImpl = GetFloatVectorVolumeImpl())
	{
		if (TSharedPtr<const FDataflowBoolVolumeImpl> MaskVolumeImpl = MaskVolume ? MaskVolume->GetBoolVolumeImpl() : MakeShared<FDataflowBoolVolumeImpl>())
		{
			TSharedPtr<FDataflowFloatVolumeImpl> DivergenceVolume = FloatVectorVolumeImpl->ComputeDivergence(OutputName, CustomName, MaskVolumeImpl);

			return FDataflowFloatVolume(DivergenceVolume);
		}
	}

	return FDataflowFloatVolume();
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume FDataflowFloatVectorVolume::ComputeMagnitude(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVectorVolumeImpl> FloatVectorVolumeImpl = GetFloatVectorVolumeImpl())
	{
		if (TSharedPtr<const FDataflowBoolVolumeImpl> MaskVolumeImpl = MaskVolume ? MaskVolume->GetBoolVolumeImpl() : MakeShared<FDataflowBoolVolumeImpl>())
		{
			TSharedPtr<FDataflowFloatVolumeImpl> LengthVolume = FloatVectorVolumeImpl->ComputeMagnitude(OutputName, CustomName, MaskVolumeImpl);
			return FDataflowFloatVolume(LengthVolume);
		}
	}

	return FDataflowFloatVolume();
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVectorVolume FDataflowFloatVectorVolume::ComputeCurl(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVectorVolumeImpl> FloatVectorVolumeImpl = GetFloatVectorVolumeImpl())
	{
		if (TSharedPtr<const FDataflowBoolVolumeImpl> MaskVolumeImpl = MaskVolume ? MaskVolume->GetBoolVolumeImpl() : MakeShared<FDataflowBoolVolumeImpl>())
		{
			TSharedPtr<FDataflowFloatVectorVolumeImpl> CurlVolume = FloatVectorVolumeImpl->ComputeCurl(OutputName, CustomName, MaskVolumeImpl);
			return FDataflowFloatVectorVolume(CurlVolume);
		}
	}

	return FDataflowFloatVectorVolume();
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVectorVolume FDataflowFloatVectorVolume::ComputeNormalize(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVectorVolumeImpl> FloatVectorVolumeImpl = GetFloatVectorVolumeImpl())
	{
		if (TSharedPtr<const FDataflowBoolVolumeImpl> MaskVolumeImpl = MaskVolume ? MaskVolume->GetBoolVolumeImpl() : MakeShared<FDataflowBoolVolumeImpl>())
		{
			TSharedPtr<FDataflowFloatVectorVolumeImpl> NormalVolume = FloatVectorVolumeImpl->ComputeNormalize(OutputName, CustomName, MaskVolumeImpl);
			return FDataflowFloatVectorVolume(NormalVolume);
		}
	}

	return FDataflowFloatVectorVolume();
}

