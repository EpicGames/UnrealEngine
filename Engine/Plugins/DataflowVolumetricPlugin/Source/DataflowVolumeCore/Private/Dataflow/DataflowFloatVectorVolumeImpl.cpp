// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowFloatVectorVolumeImpl.h"
#include "Dataflow/DataflowBoolVolumeImpl.h"
#include <Dataflow/DataflowBoolVolume.h>
#include "Engine/VolumeTexture.h"
#include "Dataflow/DataflowVolumeUtils.h"

namespace UE::DataflowVolume::Private
{
	/* --------------------------------------------------------------------------------------------------------------- */
	/* FDataflowFloatVectorVolumeImpl */
	/* --------------------------------------------------------------------------------------------------------------- */
	void FDataflowFloatVectorVolumeImpl::SetGrid(const openvdb::VectorGrid::Ptr& InFloatVectorGrid, const openvdb::VecType InVecType, const FString& InGridName)
	{
		FDataflowVolumeImpl::SetGrid(InFloatVectorGrid, InGridName);

		FloatVectorGrid = InFloatVectorGrid;
		FloatVectorGrid->setVectorType(InVecType);
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	openvdb::VectorGrid::Ptr FDataflowFloatVectorVolumeImpl::GetGrid() const
	{
		return FloatVectorGrid;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FString FDataflowFloatVectorVolumeImpl::GetGridType() const
	{
		if (FloatVectorGrid)
		{
			std::string GridType = FloatVectorGrid->gridType();
			return FString(GridType.c_str());
		}

		return {};
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	/// The type of a vector determines how transforms are applied to it:
	/// 
	/// Invariant
	/// Does not transform (e.g., tuple, uvw, color)
	///
	/// Covariant
	/// Apply inverse-transpose transformation: @e w = 0, ignores translation
	///     (e.g., gradient/normal)
	///
	/// Covariant Normalize
	/// Apply inverse-transpose transformation: @e w = 0, ignores translation,
	///     vectors are renormalized (e.g., unit normal)
	///
	/// Contravariant Relative
	/// Apply "regular" transformation: @e w = 0, ignores translation
	///     (e.g., displacement, velocity, acceleration)
	///
	/// Contravariant Absolute
	/// Apply "regular" transformation: @e w = 1, vector translates (e.g., position)
	/// 
	FString FDataflowFloatVectorVolumeImpl::GetVectorType() const
	{
		if (FloatVectorGrid)
		{
			FString VectorType;
			switch (FloatVectorGrid->getVectorType()) {
			case openvdb::VEC_INVARIANT:
				VectorType = TEXT("Invariant");
				break;
			case openvdb::VEC_COVARIANT:
				VectorType = TEXT("Covariant");
				break;
			case openvdb::VEC_COVARIANT_NORMALIZE:
				VectorType = TEXT("Covariant Normalize");
				break;
			case openvdb::VEC_CONTRAVARIANT_RELATIVE:
				VectorType = TEXT("Contravariant Relative");
				break;
			case openvdb::VEC_CONTRAVARIANT_ABSOLUTE:
				VectorType = TEXT("Contravariant Absolute");
				break;

			default:
				break;
			}

			return VectorType;
		}

		return {};
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FString FDataflowFloatVectorVolumeImpl::VolumeInfo() const
	{
		FString BaseString = FDataflowVolumeImpl::VolumeInfo();

		FString OutputStr;
		OutputStr.Appendf(TEXT("\n----------------------------------------\n"));
		OutputStr += BaseString;

		FString GridType = this->GetGridType();
		FString VectorType = this->GetVectorType();

		OutputStr.Appendf(TEXT("Type: %s\nVector Type: %s\n"),
			*GridType,
			*VectorType);

		OutputStr.Appendf(TEXT("----------------------------------------\n"));

		return OutputStr;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVectorVolumeImpl::VolumeSample(
		const TArray<FVector>& InPoints,
		TArray<FVector>& OutValues) const
	{
		using namespace openvdb;

		if (this->FloatVectorGrid->empty())
		{
			return;
		}

		openvdb::VectorGrid::ConstAccessor Accessor = this->FloatVectorGrid->getConstAccessor();

		openvdb::tools::GridSampler<openvdb::VectorGrid::ConstAccessor, openvdb::tools::BoxSampler>
			fastSampler(Accessor, this->FloatVectorGrid->transform());

		int32 Idx = 0;
		for (const FVector& Point : InPoints)
		{
			openvdb::Vec3f SampledValue = fastSampler.wsSample(openvdb::Vec3R(Point.X, Point.Y, Point.Z));
			OutValues[Idx++] = FVector(SampledValue.x(), SampledValue.y(), SampledValue.z());
		}

		return;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVectorVolumeImpl::GetActiveVoxels(const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly) const
	{
		if (FloatVectorGrid)
		{
			UE::DataflowVolumeUtils::GetActiveVoxels<openvdb::VectorGrid>(FloatVectorGrid, InIsovalue, OutActiveVoxels, bInteriorMaskOnly);
		}
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FBox FDataflowFloatVectorVolumeImpl::GetVolumeBoundingBox() const
	{
		return this->GetActiveVoxelBoundingBox();
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVolumeImpl> FDataflowFloatVectorVolumeImpl::ComputeDivergence(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const
	{
		using namespace openvdb;

		constexpr openvdb::GridClass GridClass = openvdb::GRID_UNKNOWN;
		const FString GridName = "Divergence";

		TSharedPtr<FDataflowFloatVolumeImpl> FloatVolumeImpl = MakeShared<FDataflowFloatVolumeImpl>();

		if (this->FloatVectorGrid->empty())
		{
			FloatVolumeImpl->SetGrid(openvdb::FloatGrid::Ptr(), GridClass, GridName);
			return FloatVolumeImpl;
		}

		FString NewGridName;
		if (OutputName == EDataflowVolumeAnalysisOutputName::AppendOperationName)
		{
			const FString CurrentGridName(this->FloatVectorGrid->getName().c_str());
			NewGridName = CurrentGridName + "_" + GridName;
		}
		else if (OutputName == EDataflowVolumeAnalysisOutputName::CustomName)
		{
			NewGridName = CustomName;
		}

		// TODO: Implement this
//			UE::DataflowVolumeUtils::FInterrupter NullInterrupter;

		openvdb::FloatGrid::Ptr DivergenceGrid;

		if (InMaskVolumeImpl->GetGrid())
		{
			openvdb::BoolGrid RegionMask;

			RegionMask.setTransform(this->FloatVectorGrid->transform().copy());

			openvdb::tools::resampleToMatch<openvdb::tools::PointSampler>(*InMaskVolumeImpl->GetGrid().get(), RegionMask);

			openvdb::tools::Divergence<openvdb::VectorGrid, openvdb::BoolGrid> DivergenceTool(*this->FloatVectorGrid.get(), RegionMask);
			DivergenceGrid = DivergenceTool.process(/*threaded*/true);
		}
		else
		{
			openvdb::tools::Divergence<openvdb::VectorGrid> DivergenceTool(*this->FloatVectorGrid.get());
			DivergenceGrid = DivergenceTool.process(/*threaded*/true);
		}

		FloatVolumeImpl->SetGrid(DivergenceGrid, GridClass, NewGridName);

		return FloatVolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVolumeImpl> FDataflowFloatVectorVolumeImpl::ComputeMagnitude(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const
	{
		using namespace openvdb;

		constexpr openvdb::GridClass GridClass = openvdb::GRID_UNKNOWN;
		const FString GridName = "Magnitude";

		TSharedPtr<FDataflowFloatVolumeImpl> FloatVolumeImpl = MakeShared<FDataflowFloatVolumeImpl>();

		if (this->FloatVectorGrid->empty())
		{
			FloatVolumeImpl->SetGrid(openvdb::FloatGrid::Ptr(), GridClass, GridName);
			return FloatVolumeImpl;
		}

		FString NewGridName;
		if (OutputName == EDataflowVolumeAnalysisOutputName::AppendOperationName)
		{
			const FString CurrentGridName(this->FloatVectorGrid->getName().c_str());
			NewGridName = CurrentGridName + "_" + GridName;
		}
		else if (OutputName == EDataflowVolumeAnalysisOutputName::CustomName)
		{
			NewGridName = CustomName;
		}

		// TODO: Implement this
	//			UE::DataflowVolumeUtils::FInterrupter NullInterrupter;

		openvdb::FloatGrid::Ptr MagnitudeGrid;
		if (InMaskVolumeImpl->GetGrid())
		{
			openvdb::BoolGrid RegionMask;

			RegionMask.setTransform(this->FloatVectorGrid->transform().copy());

			openvdb::tools::resampleToMatch<openvdb::tools::PointSampler>(*InMaskVolumeImpl->GetGrid().get(), RegionMask);

			openvdb::tools::Magnitude <openvdb::VectorGrid, openvdb::BoolGrid> MagnitudeTool(*this->FloatVectorGrid.get(), RegionMask);
			MagnitudeGrid = MagnitudeTool.process(/*threaded*/true);
		}
		else
		{
			openvdb::tools::Magnitude <openvdb::VectorGrid> MagnitudeTool(*this->FloatVectorGrid.get());
			MagnitudeGrid = MagnitudeTool.process(/*threaded*/true);
		}

		FloatVolumeImpl->SetGrid(MagnitudeGrid, GridClass, NewGridName);

		return FloatVolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVectorVolumeImpl> FDataflowFloatVectorVolumeImpl::ComputeCurl(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const
	{
		using namespace openvdb;

		constexpr openvdb::VecType VectorGridType = openvdb::VEC_COVARIANT;
		const FString GridName = "Curl";

		TSharedPtr<FDataflowFloatVectorVolumeImpl> FloatVectorVolumeImpl = MakeShared<FDataflowFloatVectorVolumeImpl>();

		if (this->FloatVectorGrid->empty())
		{
			FloatVectorVolumeImpl->SetGrid(openvdb::VectorGrid::Ptr(), VectorGridType, GridName);
			return FloatVectorVolumeImpl;
		}

		FString NewGridName;
		if (OutputName == EDataflowVolumeAnalysisOutputName::AppendOperationName)
		{
			const FString CurrentGridName(this->FloatVectorGrid->getName().c_str());
			NewGridName = CurrentGridName + "_" + GridName;
		}
		else if (OutputName == EDataflowVolumeAnalysisOutputName::CustomName)
		{
			NewGridName = CustomName;
		}

		// TODO: Implement this
		//			UE::DataflowVolumeUtils::FInterrupter NullInterrupter;

		openvdb::VectorGrid::Ptr CurlVectorGrid;

		if (InMaskVolumeImpl->GetGrid())
		{
			openvdb::BoolGrid RegionMask;

			RegionMask.setTransform(this->FloatVectorGrid->transform().copy());

			openvdb::tools::resampleToMatch<openvdb::tools::PointSampler>(*InMaskVolumeImpl->GetGrid().get(), RegionMask);

			openvdb::tools::Curl<openvdb::VectorGrid, openvdb::BoolGrid> CurlTool(*this->FloatVectorGrid.get(), RegionMask);
			CurlVectorGrid = CurlTool.process(/*threaded*/true);
		}
		else
		{
			openvdb::tools::Curl<openvdb::VectorGrid> CurlTool(*this->FloatVectorGrid.get());
			CurlVectorGrid = CurlTool.process(/*threaded*/true);
		}

		FloatVectorVolumeImpl->SetGrid(CurlVectorGrid, VectorGridType, NewGridName);

		return FloatVectorVolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVectorVolumeImpl> FDataflowFloatVectorVolumeImpl::ComputeNormalize(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const
	{
		using namespace openvdb;

		constexpr openvdb::VecType VectorGridType = openvdb::VEC_COVARIANT;
		const FString GridName = "Normalize";

		TSharedPtr<FDataflowFloatVectorVolumeImpl> FloatVectorVolumeImpl = MakeShared<FDataflowFloatVectorVolumeImpl>();

		if (FloatVectorGrid->empty())
		{
			FloatVectorVolumeImpl->SetGrid(openvdb::VectorGrid::Ptr(), VectorGridType, GridName);
			return FloatVectorVolumeImpl;
		}

		FString NewGridName;
		if (OutputName == EDataflowVolumeAnalysisOutputName::AppendOperationName)
		{
			const FString CurrentGridName(this->FloatVectorGrid->getName().c_str());
			NewGridName = CurrentGridName + "_" + "GridName";
		}
		else if (OutputName == EDataflowVolumeAnalysisOutputName::CustomName)
		{
			NewGridName = CustomName;
		}

		// TODO: Implement this
		//			UE::DataflowVolumeUtils::FInterrupter NullInterrupter;

		openvdb::VectorGrid::Ptr NormalVectorGrid;
		if (InMaskVolumeImpl->GetGrid())
		{
			openvdb::BoolGrid RegionMask;

			RegionMask.setTransform(this->FloatVectorGrid->transform().copy());

			openvdb::tools::resampleToMatch<openvdb::tools::PointSampler>(*InMaskVolumeImpl->GetGrid().get(), RegionMask);

			openvdb::tools::Normalize<openvdb::VectorGrid, openvdb::BoolGrid> NormalizeTool(*this->FloatVectorGrid.get(), RegionMask);
			NormalVectorGrid = NormalizeTool.process(/*threaded*/true);
		}
		else
		{
			openvdb::tools::Normalize<openvdb::VectorGrid> NormalizeTool(*this->FloatVectorGrid.get());
			NormalVectorGrid = NormalizeTool.process(/*threaded*/true);
		}

		FloatVectorVolumeImpl->SetGrid(NormalVectorGrid, VectorGridType, NewGridName);

		return FloatVectorVolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	bool FDataflowFloatVectorVolumeImpl::CreateVolumeTexture(UVolumeTexture* InVolumeTexture) const
	{
		if (InVolumeTexture == nullptr)
		{
			return false;
		}

		const EPixelFormat PixelFormat = EPixelFormat::PF_FloatRGB;

		if (!FloatVectorGrid->empty())
		{
			openvdb::Coord ActiveVoxelDim = FloatVectorGrid->evalActiveVoxelDim();
			openvdb::CoordBBox Bbox = FloatVectorGrid->evalActiveVoxelBoundingBox();
			const int32 XMin = Bbox.min().x();
			const int32 YMin = Bbox.min().y();
			const int32 ZMin = Bbox.min().z();

			openvdb::VectorGrid::Accessor Accessor = FloatVectorGrid->getAccessor();

			auto QueryVoxel = [&Accessor, &XMin, &YMin, &ZMin](int32 PosX, int32 PosY, int32 PosZ, void* Value)
			{
				openvdb::Coord Coord(PosX + XMin, PosY + YMin, PosZ + ZMin);
				FFloat16* const Voxel = static_cast<FFloat16*>(Value);

				if (Accessor.isValueOn(Coord))
				{
					Voxel[0].Set(Accessor.getValue(Coord).x());
					Voxel[1].Set(Accessor.getValue(Coord).y());
					Voxel[2].Set(Accessor.getValue(Coord).z());
					Voxel[3].Set(1.f);
				}
				else
				{
					Voxel[0].Set(0.f);
					Voxel[1].Set(0.f);
					Voxel[2].Set(0.f);
					Voxel[3].Set(0.f);
				}
			};

			const bool bSuccess = InVolumeTexture->UpdateSourceFromFunction(QueryVoxel, ActiveVoxelDim[0], ActiveVoxelDim[1], ActiveVoxelDim[2], TSF_RGBA16F);

			if (!bSuccess)
			{
				return false;
			}

			return true;
		}

		return false;
	}
}

