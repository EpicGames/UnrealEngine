// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowVolumeImpl.h"
#include "Dataflow/DataflowFloatVolumeImpl.h"
#include "Dataflow/DataflowBoolVolumeImpl.h"
#include "Dataflow/OpenVDB.h"
#include "Dataflow/DataflowVolumeNodeEnums.h"

namespace UE::DataflowVolume::Private
{
	class FDataflowFloatVolumeImpl;
	class FDataflowBoolVolumeImpl;

	/* --------------------------------------------------------------------------------------------------------------- */
	/* FDataflowFloatVectorVolumeImpl */
	/* --------------------------------------------------------------------------------------------------------------- */
	class FDataflowFloatVectorVolumeImpl : public FDataflowVolumeImpl
	{
	private:
		openvdb::VectorGrid::Ptr FloatVectorGrid = nullptr;

	public:
		// Virtual functions
		DATAFLOWVOLUMECORE_API void GetActiveVoxels(const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly = false) const;
		virtual FBox GetVolumeBoundingBox() const override;
		virtual bool CreateVolumeTexture(UVolumeTexture* InVolumeTexture) const override;

		void SetGrid(const openvdb::VectorGrid::Ptr& InFloatVectorGrid, const openvdb::VecType InVecType, const FString& InGridName = "Gradient");

		openvdb::VectorGrid::Ptr GetGrid() const;

		FString GetGridType() const;

		FString GetVectorType() const;

		FString VolumeInfo() const;

		void VolumeSample(const TArray<FVector>& InPoints, TArray<FVector>& OutValues) const;

		TSharedPtr<FDataflowFloatVolumeImpl> ComputeDivergence(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const;
		TSharedPtr<FDataflowFloatVolumeImpl> ComputeMagnitude(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const;
		TSharedPtr<FDataflowFloatVectorVolumeImpl> ComputeCurl(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const;
		TSharedPtr<FDataflowFloatVectorVolumeImpl> ComputeNormalize(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const;
	};
}
