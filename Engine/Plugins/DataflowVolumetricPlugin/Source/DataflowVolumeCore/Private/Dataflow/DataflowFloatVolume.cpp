// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowFloatVolume.h"
#include "Dataflow/DataflowIntVolume.h"
#include "Dataflow/DataflowBoolVolume.h"
#include "Dataflow/DataflowFloatVolumeImpl.h"
#include "Dataflow/DataflowFloatVectorVolume.h"
#include "Dataflow/DataflowVolumeUtils.h"
#include "MeshDescription.h"
#include "MeshAttributes.h"
#include "MeshAttributeArray.h"
#include "Dataflow/OpenVDB.h"

/* --------------------------------------------------------------------------------------------------------------- */
/* FDataflowFloatVolume Implementation */
/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume::FDataflowFloatVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowFloatVolumeImpl> InFloatVolume)
	: FDataflowVolume(InFloatVolume, FDataflowFloatVolume::TypeName)
{
	static_assert(sizeof(FDataflowVolume) == sizeof(*this));
}

/* --------------------------------------------------------------------------------------------------------------- */

TSharedPtr<const UE::DataflowVolume::Private::FDataflowFloatVolumeImpl> FDataflowFloatVolume::GetFloatVolumeImpl() const
{
	return StaticCastSharedPtr<const UE::DataflowVolume::Private::FDataflowFloatVolumeImpl>(Volume);
}

/* --------------------------------------------------------------------------------------------------------------- */

TSharedPtr<UE::DataflowVolume::Private::FDataflowFloatVolumeImpl> FDataflowFloatVolume::GetFloatVolumeImpl()
{
	return StaticCastSharedPtr<UE::DataflowVolume::Private::FDataflowFloatVolumeImpl>(Volume);
}

/* --------------------------------------------------------------------------------------------------------------- */

FVector FDataflowFloatVolume::GetVoxelSize() const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		return FloatVolumeImpl->GetVoxelSize();
	}

	return {};
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume FDataflowFloatVolume::CreateSphereSDF(
	const float InVoxelSize, 
	const float InRadius, 
	const FVector& InCenter)
{
	using namespace UE::DataflowVolume::Private;

	TSharedPtr<FDataflowFloatVolumeImpl> VolumeImpl = FDataflowFloatVolumeImpl::CreateSphereSDF(InVoxelSize, InRadius, InCenter);
	return FDataflowFloatVolume(VolumeImpl);
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume FDataflowFloatVolume::CreateCubeSDF(
	const float InScale,
	const FVector& InCenter,
	const float InVoxelSize)
{
	using namespace UE::DataflowVolume::Private;

	TSharedPtr<FDataflowFloatVolumeImpl> VolumeImpl = FDataflowFloatVolumeImpl::CreateCubeSDF(InScale, InCenter, InVoxelSize);
	return FDataflowFloatVolume(VolumeImpl);
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume FDataflowFloatVolume::CreatePlatonicSolidSDF(const int32 InFaceCount,
	const float InScale,
	const FVector& InCenter,
	const float InVoxelSize)
{
	using namespace UE::DataflowVolume::Private;

	TSharedPtr<FDataflowFloatVolumeImpl> VolumeImpl = FDataflowFloatVolumeImpl::CreatePlatonicSolidSDF(InFaceCount, InScale, InCenter, InVoxelSize);
	return FDataflowFloatVolume(VolumeImpl);
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume FDataflowFloatVolume::CreateVolumeFromMeshDescription(
	const FMeshDescription& InMeshDescription,
	const FTransform& InTransform,
	const float InVoxelSize,
	const FDataflowVolume& InReferenceVolume,
	const EDataflowVolumeOutputType InOutputType,
	const FString& InGridName,
	const bool InUseWorldSpaceUnits,
	const float InExteriorBand,
	const float InInteriorBand,
	const int32 InExteriorBandVoxels,
	const int32 InInteriorBandVoxels,
	const bool bFillInterior,
	const bool bPreserveHoles,
	const float InIsoValue,
	FDataflowIntVolume& OutFaceIndexVolume)
{
	using namespace UE::DataflowVolume::Private;

	TSharedPtr<FDataflowIntVolumeImpl> FaceIndexVolumeImpl;

	TSharedPtr<FDataflowFloatVolumeImpl> VolumeImpl = FDataflowFloatVolumeImpl::CreateVolumeFromMeshDescription(
		InMeshDescription,
		InTransform,
		InVoxelSize,
		InReferenceVolume.GetVolume(),
		InOutputType,
		InGridName,
		InUseWorldSpaceUnits,
		InExteriorBand,
		InInteriorBand,
		InExteriorBandVoxels,
		InInteriorBandVoxels,
		bFillInterior,
		bPreserveHoles,
		InIsoValue,
		FaceIndexVolumeImpl);
	
	OutFaceIndexVolume.SetIntVolume(FaceIndexVolumeImpl);

	return FDataflowFloatVolume(VolumeImpl);
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowFloatVolume::VolumeToSpheres(
	const int32 InMinSphereCount,
	const int32 InMaxSphereCount,
	const bool InOverlapping,
	const float InMinRadius,
	const float InMaxRadius,
	const float InIsoValue,
	const int32 InInstanceCount,
	TArray<FVector>& OutSphereCenters,
	TArray<float>& OutSphereRadii) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		FloatVolumeImpl->VolumeToSpheres(
			InMinSphereCount,
			InMaxSphereCount,
			InOverlapping,
			InMinRadius,
			InMaxRadius,
			InIsoValue,
			InInstanceCount,
			OutSphereCenters,
			OutSphereRadii);
	}
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowFloatVolume::VolumeToSpheresImproved(
	const int32 InMinSphereCount,
	const int32 InMaxSphereCount,
	const bool InOverlapping,
	const float InMinRadius,
	const float InMaxRadius,
	const float InIsoValue,
	const int32 InInstanceCount,
	const int32 InRandomSeed,
	const EDataflowVolumeScatterType InScatterType,
	const float InSpread,
	const float InMinNumberOfPointsPerVoxel,
	const float InMaxNumberOfPointsPerVoxel,
	TArray<FVector>& OutSphereCenters,
	TArray<float>& OutSphereRadii) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		FloatVolumeImpl->VolumeToSpheresImproved(
			InMinSphereCount,
			InMaxSphereCount,
			InOverlapping,
			InMinRadius,
			InMaxRadius,
			InIsoValue,
			InInstanceCount,
			InRandomSeed,
			InScatterType,
			InSpread,
			InMinNumberOfPointsPerVoxel,
			InMaxNumberOfPointsPerVoxel,
			OutSphereCenters,
			OutSphereRadii);
	}
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume FDataflowFloatVolume::CovertSDFToFogVolume(
	const bool InPruneTolerance,
	const float InTolerance,
	const bool InFloodFillOutput,
	const bool InActivateInterior) const
{
	using namespace UE::DataflowVolume::Private;
	TSharedPtr<FDataflowFloatVolumeImpl> ResultSDF;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		ResultSDF = FloatVolumeImpl->CovertSDFToFogVolume(
			InPruneTolerance,
			InTolerance,
			InFloodFillOutput,
			InActivateInterior);
	}

	return FDataflowFloatVolume(ResultSDF);
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume FDataflowFloatVolume::CovertFogVolumeToSDF(
	const float InFogIsoValue,
	const bool InPruneTolerance,
	const float InTolerance,
	const bool InFloodFillOutput,
	const bool InActivateInterior) const
{
	using namespace UE::DataflowVolume::Private;
	TSharedPtr<FDataflowFloatVolumeImpl> ResultSDF;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		ResultSDF = FloatVolumeImpl->CovertFogVolumeToSDF(
			InFogIsoValue,
			InPruneTolerance,
			InTolerance,
			InFloodFillOutput,
			InActivateInterior);
	}

	return FDataflowFloatVolume(ResultSDF);
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowFloatVolume::ConvertVolumeToMeshDescription(
	const float InIsoValue,
	const float InAdaptivity,
	FMeshDescription& OutMeshDescription) const
{
	using namespace UE::DataflowVolume::Private;
	TSharedPtr<FDataflowFloatVolumeImpl> ResultSDF;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		FloatVolumeImpl->ConvertVolumeToMeshDescription(InIsoValue, InAdaptivity, OutMeshDescription);
	}
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowFloatVolume::UniformVolumeScatter(
	const int32 InMinNumberOfPoints,
	const int32 InMaxNumberOfPoints,
	const int32 InRandomSeed,
	const float InIsoValue,
	const float InSpread,
	TArray<FVector>& OutPoints) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		FloatVolumeImpl->UniformVolumeScatter(
			InMinNumberOfPoints,
			InMaxNumberOfPoints,
			InRandomSeed,
			InIsoValue,
			InSpread,
			OutPoints);
	}
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowFloatVolume::DenseUniformVolumeScatter(
	const float InMinNumberOfPointsPerVoxel,
	const float InMaxNumberOfPointsPerVoxel,
	const int32 InRandomSeed,
	const float InIsoValue,
	const float InSpread,
	TArray<FVector>& OutPoints) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		FloatVolumeImpl->DenseUniformVolumeScatter(
			InMinNumberOfPointsPerVoxel,
			InMaxNumberOfPointsPerVoxel,
			InRandomSeed,
			InIsoValue,
			InSpread,
			OutPoints);
	}
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowFloatVolume::NonUniformVolumeScatter(
	const float InMinNumberOfPointsPerVoxel,
	const float InMaxNumberOfPointsPerVoxel,
	const int32 InRandomSeed,
	const float InIsoValue,
	const float InSpread,
	TArray<FVector>& OutPoints) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		FloatVolumeImpl->NonUniformVolumeScatter(
			InMinNumberOfPointsPerVoxel,
			InMaxNumberOfPointsPerVoxel,
			InRandomSeed,
			InIsoValue,
			InSpread,
			OutPoints);
	}
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowFloatVolume::VolumeSample(
	const TArray<FVector>& InPoints,
	TArray<float>& OutValues) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		FloatVolumeImpl->VolumeSample(InPoints, OutValues);
	}
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVectorVolume FDataflowFloatVolume::ComputeGradient(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		if (TSharedPtr<const FDataflowBoolVolumeImpl> MaskVolumeImpl = MaskVolume ? MaskVolume->GetBoolVolumeImpl() : MakeShared<FDataflowBoolVolumeImpl>())
		{
			TSharedPtr<FDataflowFloatVectorVolumeImpl> GradientVolume = FloatVolumeImpl->ComputeGradient(OutputName, CustomName, MaskVolumeImpl);
			return FDataflowFloatVectorVolume(GradientVolume);
		}
	}

	return FDataflowFloatVectorVolume();
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume FDataflowFloatVolume::ComputeCurvature(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		if (TSharedPtr<const FDataflowBoolVolumeImpl> MaskVolumeImpl = MaskVolume ? MaskVolume->GetBoolVolumeImpl() : MakeShared<FDataflowBoolVolumeImpl>())
		{
			TSharedPtr<FDataflowFloatVolumeImpl> CurvatureVolume = FloatVolumeImpl->ComputeCurvature(OutputName, CustomName, MaskVolumeImpl);
			return FDataflowFloatVolume(CurvatureVolume);
		}
	}

	return FDataflowFloatVolume();
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume FDataflowFloatVolume::ComputeLaplacian(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		if (TSharedPtr<const FDataflowBoolVolumeImpl> MaskVolumeImpl = MaskVolume ? MaskVolume->GetBoolVolumeImpl() : MakeShared<FDataflowBoolVolumeImpl>())
		{
			TSharedPtr<FDataflowFloatVolumeImpl> LaplacianVolume = FloatVolumeImpl->ComputeLaplacian(OutputName, CustomName, MaskVolumeImpl);
			return FDataflowFloatVolume(LaplacianVolume);
		}
	}

	return FDataflowFloatVolume();
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVectorVolume FDataflowFloatVolume::ComputeClosestPoint(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, const FDataflowBoolVolume* MaskVolume) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		if (TSharedPtr<const FDataflowBoolVolumeImpl> MaskVolumeImpl = MaskVolume ? MaskVolume->GetBoolVolumeImpl() : MakeShared<FDataflowBoolVolumeImpl>())
		{
			TSharedPtr<FDataflowFloatVectorVolumeImpl> ClosestPointVolume = FloatVolumeImpl->ComputeClosestPoint(OutputName, CustomName, MaskVolumeImpl);
			return FDataflowFloatVectorVolume(ClosestPointVolume);
		}
	}

	return FDataflowFloatVectorVolume();
}

/* --------------------------------------------------------------------------------------------------------------- */

FDataflowFloatVolume FDataflowFloatVolume::VolumeCombine(
	const FDataflowFloatVolume& InVolumeB,
	const EDataflowVolumeSDFCombineOperation InOperation,
	const float InMultiplierA,
	const float InMultiplierB,
	const EDataflowVolumeSDFCombineResample InResample,
	const EDataflowVolumeSDFCombineInterpolation InInterpolation,
	const bool InPruneDegenerateTiles,
	const float InPruneTol) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeAImpl = GetFloatVolumeImpl())
	{
		if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeBImpl = InVolumeB.GetFloatVolumeImpl())
		{
			TSharedPtr<FDataflowFloatVolumeImpl> ResultVolumeImpl = FloatVolumeAImpl->VolumeCombine(
				FloatVolumeBImpl,
				InOperation,
				InMultiplierA,
				InMultiplierB,
				InResample,
				InInterpolation,
				InPruneDegenerateTiles,
				InPruneTol);

			return FDataflowFloatVolume(ResultVolumeImpl);
		}
	}

	return FDataflowFloatVolume();
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowFloatVolume::VolumeToSpheresWithRelaxation(
	const int32 InPointCount,
	const int32 InScatterRandomSeed,
	const float InMinRadius,
	const float InMaxRadius,
	const int32 InRandomSeed,
	const float InIsoValue,
	const float InSpread,
	const bool InRemoveSpheres,
	const bool InApplyRelaxation,
	const int32 InSteps,
	const float InStepScalar,
	const bool InCorrectAgainstSurface,
	const float InDistanceThreshold,
	TArray<FSphere>& OutSpheres) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowFloatVolumeImpl> FloatVolumeImpl = GetFloatVolumeImpl())
	{
		FloatVolumeImpl->VolumeToSpheresWithRelaxation(
			InPointCount,
			InScatterRandomSeed,
			InMinRadius,
			InMaxRadius,
			InRandomSeed,
			InIsoValue,
			InSpread,
			InRemoveSpheres,
			InApplyRelaxation,
			InSteps,
			InStepScalar,
			InCorrectAgainstSurface,
			InDistanceThreshold,
			OutSpheres);
	}
}






