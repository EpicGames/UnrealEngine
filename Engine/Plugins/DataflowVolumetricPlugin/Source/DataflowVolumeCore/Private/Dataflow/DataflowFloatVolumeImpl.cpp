// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowFloatVolumeImpl.h"
#include "Dataflow/DataflowVolume.h"
#include "Dataflow/DataflowIntVolume.h"
#include "Dataflow/DataflowIntVolumeImpl.h"
#include "Dataflow/DataflowBoolVolumeImpl.h"
#include "Dataflow/DataflowVolumeUtils.h"
#include "Dataflow/DataflowVolumeAlgo.h"
#include "Engine/VolumeTexture.h"
#include "Dataflow/DataflowVolumeUtils.h"

namespace UE::DataflowVolume::Private
{
	/* --------------------------------------------------------------------------------------------------------------- */
	/* FDataflowFloatVolumeImpl */
	/* --------------------------------------------------------------------------------------------------------------- */

	openvdb::FloatGrid::Ptr FDataflowFloatVolumeImpl::GetGrid() const
	{
		return FloatGrid;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVolumeImpl::SetGrid(const openvdb::FloatGrid::Ptr& InFloatGrid, const openvdb::GridClass InGridClass, const FString& InGridName)
	{
		FDataflowVolumeImpl::SetGrid(InFloatGrid, InGridName);

		FloatGrid = InFloatGrid;
		FloatGrid->setGridClass(InGridClass);
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FString FDataflowFloatVolumeImpl::GetGridType() const
	{
		if (FloatGrid)
		{
			std::string GridType = FloatGrid->gridType();
			return FString(GridType.c_str());
		}

		return {};
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FString FDataflowFloatVolumeImpl::GetGridClass() const
	{
		if (FloatGrid)
		{
			FString GridClass;
			switch (FloatGrid->getGridClass()) {
			case openvdb::GRID_UNKNOWN:
				GridClass = TEXT("Unknown Grid Class");
				break;
			case openvdb::GRID_LEVEL_SET:
				GridClass = TEXT("Level Set");
				break;
			case openvdb::GRID_FOG_VOLUME:
				GridClass = TEXT("Fog Volume");
				break;
			case openvdb::GRID_STAGGERED:
				GridClass = TEXT("Staggered Vector Field");
				break;

			default:
				break;
			}

			return GridClass;
		}

		return {};
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FString FDataflowFloatVolumeImpl::VolumeInfo() const
	{
		FString BaseString = FDataflowVolumeImpl::VolumeInfo();

		FString OutputStr;
		OutputStr += BaseString;

		FString GridClass = this->GetGridClass();
		FString GridType = this->GetGridType();

		OutputStr.Appendf(TEXT("Class: %s\nType: %s\n"),
			*GridClass,
			*GridType);

		return OutputStr;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVolumeImpl> FDataflowFloatVolumeImpl::CreateSphereSDF(
		const float InVoxelSize,
		const float InRadius,
		const FVector& InCenter)
	{
		TSharedPtr<FDataflowFloatVolumeImpl> SDFImpl = MakeShared<FDataflowFloatVolumeImpl>();

		// Generate a level set grid
		openvdb::FloatGrid::Ptr SphereGrid = openvdb::tools::createLevelSetSphere<openvdb::FloatGrid>(
			/*radius=*/InRadius,
			/*center=*/openvdb::Vec3f((float)InCenter.X, (float)InCenter.Y, (float)InCenter.Z),
			/*voxel size=*/InVoxelSize);

		SphereGrid->insertMeta("radius", openvdb::FloatMetadata(InRadius));
		SphereGrid->insertMeta("center", openvdb::Vec3SMetadata(openvdb::Vec3s((float)InCenter.X, (float)InCenter.Y, (float)InCenter.Z)));

		SDFImpl->SetGrid(SphereGrid, openvdb::GRID_LEVEL_SET);

		return SDFImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVolumeImpl> FDataflowFloatVolumeImpl::CreateCubeSDF(const float InScale, const FVector& InCenter, const float InVoxelSize)
	{
		TSharedPtr<FDataflowFloatVolumeImpl> SDFImpl = MakeShared<FDataflowFloatVolumeImpl>();

		const openvdb::Vec3f CenterVec = openvdb::Vec3f(InCenter.X, InCenter.Y, InCenter.Z);

		// Generate a level set grid
		openvdb::FloatGrid::Ptr BoxGrid = openvdb::tools::createLevelSetCube<openvdb::FloatGrid>(
			/*scale of the cube in world units*/InScale,
			/*center of the cube in world units*/CenterVec,
			/*voxel size in world units*/InVoxelSize);

		SDFImpl->SetGrid(BoxGrid, openvdb::GRID_LEVEL_SET);

		return SDFImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVolumeImpl> FDataflowFloatVolumeImpl::CreatePlatonicSolidSDF(const int32 InFaceCount,
		const float InScale,
		const FVector& InCenter,
		const float InVoxelSize)
	{
		TSharedPtr<FDataflowFloatVolumeImpl> SDFImpl = MakeShared<FDataflowFloatVolumeImpl>();

		// Generate a level set grid
		const openvdb::Vec3f CenterVec = openvdb::Vec3f(InCenter.X, InCenter.Y, InCenter.Z);
		openvdb::FloatGrid::Ptr PlatonicGrid = openvdb::tools::createLevelSetPlatonic<openvdb::FloatGrid>(
			/*number of faces of the platonic solid, i.e. 4, 6, 8, 12 or 20*/InFaceCount,
			/*scale of the platonic solid in world units*/InScale,
			/*center of the platonic solid in world units*/CenterVec,
			/*voxel size in world units*/InVoxelSize);

		openvdb::math::Transform GridTransform = PlatonicGrid->transform();

		SDFImpl->SetGrid(PlatonicGrid, openvdb::GRID_LEVEL_SET);

		return SDFImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVolumeImpl> FDataflowFloatVolumeImpl::CreateVolumeFromMeshDescription(
		const FMeshDescription& InMeshDescription,
		const FTransform& InTransform,
		const float InVoxelSize,
		const TSharedPtr<const FDataflowVolumeImpl>& InReferenceVolumeImpl,
		const EDataflowVolumeOutputType InOutputType,
		const FString& InGridName,
		const bool InUseWorldSpaceUnits,
		const float InExteriorBand,
		const float InInteriorBand,
		const int32 InExteriorBandVoxels,
		const int32 InInteriorBandVoxels,
		const bool bFillInterior,
		const bool bPreserveHoles,
		const float InIsoValue,
		TSharedPtr<FDataflowIntVolumeImpl>& OutFaceIndexVolume)
	{
		using namespace UE::DataflowVolume::Private;

		TSharedPtr<FDataflowFloatVolumeImpl> FloatVolumeImpl = MakeShared<FDataflowFloatVolumeImpl>();

		// If there is a reference grid connected, derive the voxel size and transform
		float VoxelSize = InVoxelSize;

		// @todo(gmelich): Implement using a reference grid
		if (InReferenceVolumeImpl.IsValid() && InReferenceVolumeImpl->IsGridEmpty())
		{
		}

		UE::DataflowVolumeUtils::FPlacedMesh PlacedMesh(&InMeshDescription, InTransform);

		FMatrix LocalToVoxel = FMatrix::Identity;
		LocalToVoxel.M[0][0] = InVoxelSize;
		LocalToVoxel.M[1][1] = InVoxelSize;
		LocalToVoxel.M[2][2] = InVoxelSize;

		FVector AverageTranslation = FVector(0.0);

		auto TransformGenerator = [&LocalToVoxel, &AverageTranslation](const UE::DataflowVolumeUtils::FPlacedMesh& PlacedMesh)->openvdb::Mat4R
			{
				FTransform MeshXForm = PlacedMesh.Transform;
				MeshXForm.AddToTranslation(-AverageTranslation);
				FMatrix TransformMatrix = MeshXForm.ToMatrixWithScale().Inverse();

				TransformMatrix = LocalToVoxel * TransformMatrix;
				double* data = &TransformMatrix.M[0][0];
				openvdb::Mat4R VDBMatDouble(data);
				// NB: rounding errors in the inverse may have resulted in error in this col.
				// openvdb explicitly checks this matrix row to insure the transform is affine and will throw 
				VDBMatDouble.setCol(3, openvdb::Vec4R(0, 0, 0, 1));
				return VDBMatDouble;
			};

		openvdb::Mat4R XFormA = TransformGenerator(PlacedMesh);
		openvdb::math::Transform::Ptr VDBXForm = openvdb::math::Transform::createLinearTransform(XFormA);

		// Create adapters that understand the openVDB semantics
		UE::DataflowVolumeUtils::FMeshDescriptionAdapter Adapter(InMeshDescription, *VDBXForm);

		// target transform
		openvdb::math::Transform::Ptr TargetXForm = openvdb::math::Transform::createLinearTransform(InVoxelSize);
		//	openvdb::math::Transform::Ptr TargetXForm = openvdb::math::Transform::createLinearTransform(XFormA);

		if (VoxelSize < 1e-5)
		{
			FloatVolumeImpl->SetGrid(openvdb::FloatGrid::Ptr(), openvdb::GRID_LEVEL_SET);

			return FloatVolumeImpl;
		}

		// Compute band width
		float ExteriorBand = 0.f;
		float InteriorBand = FLT_MAX;

		if (InUseWorldSpaceUnits)
		{
			ExteriorBand = InExteriorBand / VoxelSize;
		}
		else
		{
			ExteriorBand = InExteriorBandVoxels;
		}

		if (!bFillInterior)
		{
			if (InUseWorldSpaceUnits)
			{
				InteriorBand = InInteriorBand / VoxelSize;
			}
			else
			{
				InteriorBand = InInteriorBandVoxels;
			}
		}

		// @todo(gmelich): Implement using interrupter
		// Interrupter
		UE::DataflowVolumeUtils::FInterrupter NullInterrupter;

		// Grid for storing closest face index
		openvdb::Int32Grid::Ptr FaceIndexGrid;
		FaceIndexGrid.reset(new openvdb::Int32Grid(0));

		// Compute output
		openvdb::FloatGrid::Ptr ResultGrid;

		openvdb::GridClass Class = openvdb::GridClass::GRID_UNKNOWN;
		if (InOutputType == EDataflowVolumeOutputType::SDF)
		{
			int32 Flags = 0;
			ResultGrid = openvdb::tools::meshToVolume<openvdb::FloatGrid>(NullInterrupter, Adapter, *TargetXForm, ExteriorBand, InteriorBand, Flags, FaceIndexGrid.get());
			Class = openvdb::GRID_LEVEL_SET;
		}
		else if (InOutputType == EDataflowVolumeOutputType::USDF)
		{
			int32 Flags = openvdb::tools::UNSIGNED_DISTANCE_FIELD;
			ResultGrid = openvdb::tools::meshToVolume<openvdb::FloatGrid>(NullInterrupter, Adapter, *TargetXForm, ExteriorBand, InteriorBand, Flags, FaceIndexGrid.get());
			Class = openvdb::GRID_LEVEL_SET;
		}
		else if (InOutputType == EDataflowVolumeOutputType::FogVolume)
		{
			int32 Flags = 0;
			ResultGrid = openvdb::tools::meshToVolume<openvdb::FloatGrid>(NullInterrupter, Adapter, *TargetXForm, ExteriorBand, InteriorBand, Flags, FaceIndexGrid.get());
			openvdb::tools::sdfToFogVolume(*ResultGrid);
			Class = openvdb::GRID_FOG_VOLUME;
		}

		// Set output grids
		FloatVolumeImpl->SetGrid(ResultGrid, Class, InGridName);

		OutFaceIndexVolume = MakeShared<FDataflowIntVolumeImpl>();
		if (OutFaceIndexVolume)
		{
			OutFaceIndexVolume->SetGrid(FaceIndexGrid, "FaceIndex");
		}

		return FloatVolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVolumeImpl::VolumeToSpheres(const int32 InMinSphereCount,
		const int32 InMaxSphereCount,
		const bool InOverlapping,
		const float InMinRadius,
		float InMaxRadius,
		const float InIsoValue,
		const int32 InInstanceCount,
		TArray<FVector>& OutSphereCenters,
		TArray<float>& OutSphereRadii) const
	{
		if (FloatGrid)
		{
			openvdb::FloatGrid::ConstPtr GridPtr = openvdb::gridConstPtrCast<openvdb::FloatGrid>(FloatGrid);

			std::vector<openvdb::Vec4s> Spheres;
			const openvdb::Vec2i SphereCount(InMinSphereCount, InMaxSphereCount);
			if (InMaxRadius <= InMinRadius)
			{
				InMaxRadius = InMinRadius + (float)1e-5;
			}

			openvdb::tools::fillWithSpheres(*GridPtr,
				Spheres, SphereCount, InOverlapping, InMinRadius, InMaxRadius, InIsoValue, InInstanceCount);

			const int32 NumSpheres = Spheres.size();
			OutSphereCenters.AddUninitialized(NumSpheres);
			OutSphereRadii.AddUninitialized(NumSpheres);

			for (int32 Idx = 0; Idx < NumSpheres; ++Idx)
			{
				OutSphereCenters[Idx] = FVector(Spheres[Idx].x(), Spheres[Idx].y(), Spheres[Idx].z());
				OutSphereRadii[Idx] = Spheres[Idx].w();
			}
		}
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVolumeImpl::VolumeToSpheresImproved(const int32 InMinSphereCount,
		const int32 InMaxSphereCount,
		const bool InOverlapping,
		const float InMinRadius,
		float InMaxRadius,
		const float InIsoValue,
		const int32 InInstanceCount,
		const int32 InRandomSeed,
		const EDataflowVolumeScatterType InScatterType,
		const float InSpread,
		const float InMinNumberOfPointsPerVoxel,
		const float InMaxNumberOfPointsPerVoxel,
		TArray<FVector>& OutSphereCenters,
		TArray<float>& OutSphereRadii) const
	{
		if (FloatGrid)
		{
			openvdb::FloatGrid::ConstPtr GridPtr = openvdb::gridConstPtrCast<openvdb::FloatGrid>(FloatGrid);

			std::vector<openvdb::Vec4s> Spheres;
			const openvdb::Vec2i SphereCount(InMinSphereCount, InMaxSphereCount);
			if (InMaxRadius <= InMinRadius)
			{
				InMaxRadius = InMinRadius + (float)1e-5;
			}

			FillWithSpheres(*GridPtr,
				Spheres, SphereCount, InOverlapping, InMinRadius, InMaxRadius, 
				InIsoValue, InInstanceCount, InRandomSeed, InScatterType, InSpread, InMinNumberOfPointsPerVoxel, InMaxNumberOfPointsPerVoxel);

			const int32 NumSpheres = Spheres.size();
			OutSphereCenters.AddUninitialized(NumSpheres);
			OutSphereRadii.AddUninitialized(NumSpheres);

			for (int32 Idx = 0; Idx < NumSpheres; ++Idx)
			{
				OutSphereCenters[Idx] = FVector(Spheres[Idx].x(), Spheres[Idx].y(), Spheres[Idx].z());
				OutSphereRadii[Idx] = Spheres[Idx].w();
			}
		}
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVolumeImpl> FDataflowFloatVolumeImpl::CovertSDFToFogVolume(
		const bool InPruneTolerance,
		const float InTolerance,
		const bool InFloodFillOutput,
		const bool InActivateInteriror) const
	{
		TSharedPtr<FDataflowFloatVolumeImpl> VolumeImpl = MakeShared<FDataflowFloatVolumeImpl>();

		openvdb::FloatGrid::Ptr ResultGrid = this->FloatGrid;

		// Compute
		openvdb::tools::sdfToFogVolume(*ResultGrid);
		VolumeImpl->SetGrid(ResultGrid, openvdb::GRID_FOG_VOLUME, "FogVolume");

		return VolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVolumeImpl> FDataflowFloatVolumeImpl::CovertFogVolumeToSDF(
		const float InFogIsoValue,
		const bool InPruneTolerance,
		const float InTolerance,
		const bool InFloodFillOutput,
		const bool InActivateInteriror) const
	{
		TSharedPtr<FDataflowFloatVolumeImpl> VolumeImpl = MakeShared<FDataflowFloatVolumeImpl>();

		openvdb::FloatGrid::Ptr ResultGrid = this->FloatGrid;
		//FMeshDescription MeshDescription,

		// @todo(gmelich): Fix this

		// Step 1 - Convert Fog volume to mesh
		//UE::DataflowVolumeUtils::FMixedPolyMesh TempMixedPolyMesh;
		//UE::DataflowVolumeUtils::FAOSMesh AOSMeshedVolume;

		//openvdb::tools::volumeToMesh(*this->FloatGrid, TempMixedPolyMesh.Points, TempMixedPolyMesh.Triangles, TempMixedPolyMesh.Quads, InFogIsoValue, InAdaptivity);

		//UE::DataflowVolumeUtils::MixedPolyMeshToAOSMesh(TempMixedPolyMesh, AOSMeshedVolume);

		//ConvertMesh(AOSMeshedVolume, MeshDescription);

		//// Step 2 - Convert mesh to SDF
		//float VoxelSize = ResultGrid->voxelSize()[0];

		//FMatrix LocalToVoxel = FMatrix::Identity;
		//LocalToVoxel.M[0][0] = VoxelSize;
		//LocalToVoxel.M[1][1] = VoxelSize;
		//LocalToVoxel.M[2][2] = VoxelSize;

		//UE::DataflowVolumeUtils::FPlacedMesh PlacedMesh(&MeshDescription, FTransform::Identity);
		//FVector AverageTranslation = FVector(0.0);

		//auto TransformGenerator = [&LocalToVoxel, &AverageTranslation](const UE::DataflowVolumeUtils::FPlacedMesh& PlacedMesh)->openvdb::Mat4R
		//	{
		//		FTransform MeshXForm = PlacedMesh.Transform;
		//		MeshXForm.AddToTranslation(-AverageTranslation);
		//		FMatrix TransformMatrix = MeshXForm.ToMatrixWithScale().Inverse();

		//		TransformMatrix = LocalToVoxel * TransformMatrix;
		//		double* data = &TransformMatrix.M[0][0];
		//		openvdb::Mat4R VDBMatDouble(data);
		//		// NB: rounding errors in the inverse may have resulted in error in this col.
		//		// openvdb explicitly checks this matrix row to insure the transform is affine and will throw 
		//		VDBMatDouble.setCol(3, openvdb::Vec4R(0, 0, 0, 1));
		//		return VDBMatDouble;
		//	};

		//openvdb::Mat4R XFormA = TransformGenerator(PlacedMesh);
		//openvdb::math::Transform::Ptr VDBXForm = openvdb::math::Transform::createLinearTransform(XFormA);

		//// Create adapters that understand the openVDB semantics
		//UE::DataflowVolumeUtils::FMeshDescriptionAdapter Adapter(MeshDescription, *VDBXForm);

		//openvdb::math::Transform::Ptr TargetXForm = openvdb::math::Transform::createLinearTransform(VoxelSize);

		//UE::DataflowVolumeUtils::FInterrupter NullInterrupter;

		//int32 Flags = 0;
		//ResultGrid = openvdb::tools::meshToVolume<openvdb::FloatGrid>(NullInterrupter, Adapter, *TargetXForm);

		//VolumeImpl->SetGrid(ResultGrid, openvdb::GRID_LEVEL_SET, "LevelSet");

		VolumeImpl->SetGrid(ResultGrid, openvdb::GRID_FOG_VOLUME, "FogVolume");
		return VolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVolumeImpl::CovertSDFToCollection(
		const float InIsoValue,
		const float InAdaptivity,
		FManagedArrayCollection& OutCollection) const
	{
		openvdb::FloatGrid::Ptr ResultGrid;

		// Compute
		openvdb::tools::VolumeToMesh Mesher(InIsoValue, InAdaptivity);
		Mesher(*this->FloatGrid);

		const int32 NumPoints = Mesher.pointListSize();
		const int32 NumPolys = Mesher.polygonPoolListSize();

		openvdb::tools::PointList& Points = Mesher.pointList();
		openvdb::tools::PolygonPoolList& Polys = Mesher.polygonPoolList();
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVolumeImpl::ConvertVolumeToMeshDescription(
		const float InIsoValue,
		const float InAdaptivity,
		FMeshDescription& OutMeshDescription) const
	{
		openvdb::FloatGrid::Ptr ResultGrid;

		UE::DataflowVolumeUtils::FMixedPolyMesh TempMixedPolyMesh;
		UE::DataflowVolumeUtils::FAOSMesh AOSMeshedVolume;

		openvdb::tools::volumeToMesh(*this->FloatGrid, TempMixedPolyMesh.Points, TempMixedPolyMesh.Triangles, TempMixedPolyMesh.Quads, InIsoValue, InAdaptivity);

		UE::DataflowVolumeUtils::MixedPolyMeshToAOSMesh(TempMixedPolyMesh, AOSMeshedVolume);

		ConvertMesh(AOSMeshedVolume, OutMeshDescription);
	}

	/* --------------------------------------------------------------------------------------------------------------- */


	void FDataflowFloatVolumeImpl::GetActiveVoxels(const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly) const
	{
		if (FloatGrid)
		{
			UE::DataflowVolumeUtils::GetActiveVoxels<openvdb::FloatGrid>(FloatGrid, InIsovalue, OutActiveVoxels, bInteriorMaskOnly);
		}	
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FBox FDataflowFloatVolumeImpl::GetVolumeBoundingBox() const
	{
		return this->GetActiveVoxelBoundingBox();
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	typedef openvdb::FloatGrid GridT;
	using TreeT = typename GridT::TreeType;
	using BoolTreeT = typename TreeT::template ValueConverter<bool>::Type;

	static openvdb::Grid<BoolTreeT>::Ptr GetVolumeForScatter(
		const openvdb::FloatGrid::Ptr& InFloatGrid,
		const float InIsoValue)
	{
		using namespace openvdb;

		openvdb::FloatGrid::Ptr gridPtr = InFloatGrid->copy(); // shallow copy

		float Isovalue = 0.0f;

		if (gridPtr->getGridClass() == GRID_LEVEL_SET)
		{
			// Clamp the isovalue to the level set's background value minus epsilon.
			// (In a valid narrow-band level set, all voxels, including background voxels,
			// have values less than or equal to the background value, so an isovalue
			// greater than or equal to the background value would produce a mask with
			// effectively infinite extent.)
			Isovalue = std::min(InIsoValue,
				static_cast<float>(gridPtr->background() - math::Tolerance<float>::value()));
		}
		else if (gridPtr->getGridClass() == GRID_FOG_VOLUME)
		{
			// Clamp the isovalue of a fog volume between epsilon and one,
			// again to avoid a mask with infinite extent.  (Recall that
			// fog volume voxel values vary from zero outside to one inside.)
			Isovalue = math::Clamp(InIsoValue, math::Tolerance<float>::value(), 1.f);
		}

		int32 numVoxels = gridPtr->activeVoxelCount();
		if (numVoxels < 10000)
		{
			const double scale = 1.0 / math::Cbrt(2.0 * 10000.0 / double(numVoxels));
			auto scaledXform = gridPtr->transform().copy();
			scaledXform->preScale(scale);

			auto newGridPtr = openvdb::tools::levelSetRebuild(*gridPtr, Isovalue,
				LEVEL_SET_HALF_WIDTH, LEVEL_SET_HALF_WIDTH, scaledXform.get());

			const int32 newNumVoxels = newGridPtr->activeVoxelCount();
			if (newNumVoxels > numVoxels)
			{
				gridPtr = newGridPtr;
				numVoxels = newNumVoxels;
			}
		}

		//			typedef openvdb::FloatGrid GridT;

		//			using TreeT = typename GridT::TreeType;
		//			using BoolTreeT = typename TreeT::template ValueConverter<bool>::Type;

		const TreeT& tree = gridPtr->tree();
		math::Transform transform = gridPtr->transform();

		// Compute a mask of the voxels enclosed by the isosurface.
		typename Grid<BoolTreeT>::Ptr InteriorMaskPtr;

		if (gridPtr->getGridClass() == GRID_LEVEL_SET) {
			InteriorMaskPtr = tools::sdfInteriorMask(*gridPtr, Isovalue);
		}
		else
		{
			// For non-level-set grids, the interior mask comprises the active voxels.
			InteriorMaskPtr = typename Grid<BoolTreeT>::Ptr(Grid<BoolTreeT>::create(false));
			InteriorMaskPtr->setTransform(transform.copy());
			InteriorMaskPtr->tree().topologyUnion(tree);
		}

		{
			auto& maskTree = InteriorMaskPtr->tree();
			auto copyOfTree = StaticPtrCast<BoolTreeT>(maskTree.copy());
			tools::erodeActiveValues(maskTree, /*iterations=*/1, tools::NN_FACE, tools::IGNORE_TILES);
			tools::pruneInactive(maskTree);
			if (maskTree.empty()) { InteriorMaskPtr->setTree(copyOfTree); }
		}

		return InteriorMaskPtr;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVolumeImpl::UniformVolumeScatter(
		const int32 InMinNumberOfPoints,
		const int32 InMaxNumberOfPoints,
		const int32 InRandomSeed,
		const float InIsoValue,
		const float InSpread,
		TArray<FVector>& OutPoints) const
	{
		using namespace openvdb;

		OutPoints.Empty();

		if (this->FloatGrid->empty() ||
			!(this->FloatGrid->getGridClass() == GRID_LEVEL_SET || this->FloatGrid->getGridClass() == GRID_FOG_VOLUME))
		{
			return;
		}

		openvdb::Grid<BoolTreeT>::Ptr InteriorMaskPtr = GetVolumeForScatter(this->FloatGrid, InIsoValue);

		FRandomStream RandStream(InRandomSeed);
		int32 NumPoints = InMinNumberOfPoints;
		if (InMaxNumberOfPoints > InMinNumberOfPoints)
		{
			NumPoints = RandStream.RandRange(InMinNumberOfPoints, InMaxNumberOfPoints);
		}

		using RandGen = std::mersenne_twister_engine<uint32_t, 32, 351, 175, 19,
			0xccab8ee7, 11, 0xffffffff, 7, 0x31b6ab00, 15, 0xffe50000, 17, 1812433253>; // mt11213b
		RandGen MTRand(/*seed=*/InRandomSeed);

		std::vector<Vec3R> InstancePoints;
		InstancePoints.reserve(NumPoints);
		tools::v2s_internal::PointAccessor PointAcc(InstancePoints);

		tools::UniformPointScatter<tools::v2s_internal::PointAccessor, RandGen> UniformScatter(PointAcc, Index64(NumPoints), MTRand, double(InSpread));
		UniformScatter(*InteriorMaskPtr);

		for (std::vector<Vec3R>::iterator It = InstancePoints.begin(); It != InstancePoints.end(); It++)
		{
			const FVector Position = FVector(It->x(), It->y(), It->z());
			OutPoints.Add(Position);
		}
		return;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVolumeImpl::DenseUniformVolumeScatter(
		const float InMinNumberOfPointsPerVoxel,
		const float InMaxNumberOfPointsPerVoxel,
		const int32 InRandomSeed,
		const float InIsoValue,
		const float InSpread,
		TArray<FVector>& OutPoints) const
	{
		using namespace openvdb;

		OutPoints.Empty();

		if (this->FloatGrid->empty() ||
			!(this->FloatGrid->getGridClass() == GRID_LEVEL_SET || this->FloatGrid->getGridClass() == GRID_FOG_VOLUME))
		{
			return;
		}

		openvdb::Grid<BoolTreeT>::Ptr InteriorMaskPtr = GetVolumeForScatter(this->FloatGrid, InIsoValue);

		FRandomStream RandStream(InRandomSeed);
		float NumPointsPerVoxel = InMinNumberOfPointsPerVoxel;
		if (InMaxNumberOfPointsPerVoxel > InMinNumberOfPointsPerVoxel)
		{
			NumPointsPerVoxel = RandStream.FRandRange(InMinNumberOfPointsPerVoxel, InMaxNumberOfPointsPerVoxel);
		}

		using RandGen = std::mersenne_twister_engine<uint32_t, 32, 351, 175, 19,
			0xccab8ee7, 11, 0xffffffff, 7, 0x31b6ab00, 15, 0xffe50000, 17, 1812433253>; // mt11213b
		RandGen MTRand(/*seed=*/InRandomSeed);

		std::vector<Vec3R> InstancePoints;
		//			InstancePoints.reserve(NumPoints);
		tools::v2s_internal::PointAccessor PointAcc(InstancePoints);

		tools::DenseUniformPointScatter<tools::v2s_internal::PointAccessor, RandGen> DenseUniformScatter(PointAcc, NumPointsPerVoxel, MTRand, double(InSpread));
		DenseUniformScatter(*InteriorMaskPtr);

		for (std::vector<Vec3R>::iterator It = InstancePoints.begin(); It != InstancePoints.end(); It++)
		{
			const FVector Position = FVector(It->x(), It->y(), It->z());
			OutPoints.Add(Position);
		}
		return;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVolumeImpl::NonUniformVolumeScatter(
		const float InMinNumberOfPointsPerVoxel,
		const float InMaxNumberOfPointsPerVoxel,
		const int32 InRandomSeed,
		const float InIsoValue,
		const float InSpread,
		TArray<FVector>& OutPoints) const
	{
		using namespace openvdb;

		OutPoints.Empty();

		if (this->FloatGrid->empty() ||
			!(this->FloatGrid->getGridClass() == GRID_LEVEL_SET || this->FloatGrid->getGridClass() == GRID_FOG_VOLUME))
		{
			return;
		}

		openvdb::Grid<BoolTreeT>::Ptr InteriorMaskPtr = GetVolumeForScatter(this->FloatGrid, InIsoValue);

		FRandomStream RandStream(InRandomSeed);
		float NumPointsPerVoxel = InMinNumberOfPointsPerVoxel;
		if (InMaxNumberOfPointsPerVoxel > InMinNumberOfPointsPerVoxel)
		{
			NumPointsPerVoxel = RandStream.FRandRange(InMinNumberOfPointsPerVoxel, InMaxNumberOfPointsPerVoxel);
		}

		using RandGen = std::mersenne_twister_engine<uint32_t, 32, 351, 175, 19,
			0xccab8ee7, 11, 0xffffffff, 7, 0x31b6ab00, 15, 0xffe50000, 17, 1812433253>; // mt11213b
		RandGen MTRand(/*seed=*/InRandomSeed);

		std::vector<Vec3R> InstancePoints;
		//			InstancePoints.reserve(NumPoints);
		tools::v2s_internal::PointAccessor PointAcc(InstancePoints);

		tools::NonUniformPointScatter<tools::v2s_internal::PointAccessor, RandGen> NonUniformScatter(PointAcc, NumPointsPerVoxel, MTRand, double(InSpread));
		NonUniformScatter(*InteriorMaskPtr);

		for (std::vector<Vec3R>::iterator It = InstancePoints.begin(); It != InstancePoints.end(); It++)
		{
			const FVector Position = FVector(It->x(), It->y(), It->z());
			OutPoints.Add(Position);
		}
		return;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVolumeImpl::VolumeSample(const TArray<FVector>& InPoints, TArray<float>& OutValues) const
	{
		using namespace openvdb;

		if (this->FloatGrid->empty() ||
			!(this->FloatGrid->getGridClass() == GRID_LEVEL_SET || this->FloatGrid->getGridClass() == GRID_FOG_VOLUME))
		{
			return;
		}

		openvdb::FloatGrid::ConstAccessor Accessor = this->FloatGrid->getConstAccessor();

		openvdb::tools::GridSampler<openvdb::FloatGrid::ConstAccessor, openvdb::tools::BoxSampler>
			fastSampler(Accessor, this->FloatGrid->transform());

		int32 Idx = 0;
		for (const FVector& Point : InPoints)
		{
			OutValues[Idx++] = fastSampler.wsSample(openvdb::Vec3R(Point.X, Point.Y, Point.Z));
		}

		return;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVectorVolumeImpl> FDataflowFloatVolumeImpl::ComputeGradient(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const
	{
		using namespace openvdb;

		constexpr openvdb::VecType VectorGridType = openvdb::VEC_COVARIANT;
		const FString GridName = "Gradient";

		TSharedPtr<FDataflowFloatVectorVolumeImpl> FloatVectorVolumeImpl = MakeShared<FDataflowFloatVectorVolumeImpl>();

		if (this->FloatGrid->empty())
		{
			FloatVectorVolumeImpl->SetGrid(openvdb::VectorGrid::Ptr(), VectorGridType, GridName);
			return FloatVectorVolumeImpl;
		}

		FString NewGridName;
		if (OutputName == EDataflowVolumeAnalysisOutputName::AppendOperationName)
		{
			const FString CurrentGridName(this->FloatGrid->getName().c_str());
			NewGridName = CurrentGridName + "_" + GridName;
		}
		else if (OutputName == EDataflowVolumeAnalysisOutputName::CustomName)
		{
			NewGridName = CustomName;
		}

		// @todo(gmelich): Implement using interrupter
//			UE::DataflowVolumeUtils::FInterrupter NullInterrupter;

		openvdb::VectorGrid::Ptr GradientVectorGrid;

		if (InMaskVolumeImpl->GetGrid())
		{
			openvdb::BoolGrid RegionMask;

			RegionMask.setTransform(this->FloatGrid->transform().copy());

			openvdb::tools::resampleToMatch<openvdb::tools::PointSampler>(*InMaskVolumeImpl->GetGrid().get(), RegionMask);

			openvdb::tools::Gradient<openvdb::FloatGrid, openvdb::BoolGrid> GradientTool(*this->FloatGrid.get(), RegionMask);
			GradientVectorGrid = GradientTool.process(/*threaded*/true);
		}
		else
		{
			openvdb::tools::Gradient<openvdb::FloatGrid> GradientTool(*this->FloatGrid.get());
			GradientVectorGrid = GradientTool.process(/*threaded*/true);
		}

		FloatVectorVolumeImpl->SetGrid(GradientVectorGrid, VectorGridType, NewGridName);

		return FloatVectorVolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVolumeImpl> FDataflowFloatVolumeImpl::ComputeCurvature(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const
	{
		using namespace openvdb;

		constexpr openvdb::GridClass GridClass = openvdb::GRID_UNKNOWN;
		const FString GridName = "Curvature";

		TSharedPtr<FDataflowFloatVolumeImpl> FloatVolumeImpl = MakeShared<FDataflowFloatVolumeImpl>();

		if (this->FloatGrid->empty())
		{
			FloatVolumeImpl->SetGrid(openvdb::FloatGrid::Ptr(), GridClass, GridName);
			return FloatVolumeImpl;
		}

		FString NewGridName;
		if (OutputName == EDataflowVolumeAnalysisOutputName::AppendOperationName)
		{
			const FString CurrentGridName(this->FloatGrid->getName().c_str());
			NewGridName = CurrentGridName + "_" + GridName;
		}
		else if (OutputName == EDataflowVolumeAnalysisOutputName::CustomName)
		{
			NewGridName = CustomName;
		}

		// @todo(gmelich): Implement using interrupter
//			UE::DataflowVolumeUtils::FInterrupter NullInterrupter;

		openvdb::FloatGrid::Ptr CurvatureGrid;

		if (InMaskVolumeImpl->GetGrid())
		{
			openvdb::BoolGrid RegionMask;

			RegionMask.setTransform(this->FloatGrid->transform().copy());

			openvdb::tools::resampleToMatch<openvdb::tools::PointSampler>(*InMaskVolumeImpl->GetGrid().get(), RegionMask);

			openvdb::tools::MeanCurvature<openvdb::FloatGrid, openvdb::BoolGrid> CurvatureTool(*this->FloatGrid.get(), RegionMask);
			CurvatureGrid = CurvatureTool.process(/*threaded*/true);
		}
		else
		{
			openvdb::tools::MeanCurvature<openvdb::FloatGrid> CurvatureTool(*this->FloatGrid.get());
			CurvatureGrid = CurvatureTool.process(/*threaded*/true);
		}

		FloatVolumeImpl->SetGrid(CurvatureGrid, GridClass, NewGridName);

		return FloatVolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVolumeImpl> FDataflowFloatVolumeImpl::ComputeLaplacian(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const
	{
		using namespace openvdb;

		constexpr openvdb::GridClass GridClass = openvdb::GRID_UNKNOWN;
		const FString GridName = "Laplacian";

		TSharedPtr<FDataflowFloatVolumeImpl> FloatVolumeImpl = MakeShared<FDataflowFloatVolumeImpl>();

		if (this->FloatGrid->empty())
		{
			FloatVolumeImpl->SetGrid(openvdb::FloatGrid::Ptr(), GridClass, GridName);
			return FloatVolumeImpl;
		}

		FString NewGridName;
		if (OutputName == EDataflowVolumeAnalysisOutputName::AppendOperationName)
		{
			const FString CurrentGridName(this->FloatGrid->getName().c_str());
			NewGridName = CurrentGridName + "_" + GridName;
		}
		else if (OutputName == EDataflowVolumeAnalysisOutputName::CustomName)
		{
			NewGridName = CustomName;
		}

		// @todo(gmelich): Implement using interrupter
//			UE::DataflowVolumeUtils::FInterrupter NullInterrupter;

		openvdb::FloatGrid::Ptr LaplacianGrid;

		if (InMaskVolumeImpl->GetGrid())
		{
			openvdb::BoolGrid RegionMask;

			RegionMask.setTransform(this->FloatGrid->transform().copy());

			openvdb::tools::resampleToMatch<openvdb::tools::PointSampler>(*InMaskVolumeImpl->GetGrid().get(), RegionMask);

			openvdb::tools::Laplacian<openvdb::FloatGrid, openvdb::BoolGrid> LaplacianTool(*this->FloatGrid.get(), RegionMask);
			LaplacianGrid = LaplacianTool.process(/*threaded*/true);
		}
		else
		{
			openvdb::tools::Laplacian<openvdb::FloatGrid> LaplacianTool(*this->FloatGrid.get());
			LaplacianGrid = LaplacianTool.process(/*threaded*/true);
		}

		FloatVolumeImpl->SetGrid(LaplacianGrid, GridClass, NewGridName);

		return FloatVolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	TSharedPtr<FDataflowFloatVectorVolumeImpl> FDataflowFloatVolumeImpl::ComputeClosestPoint(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const
	{
		using namespace openvdb;

		constexpr openvdb::VecType VectorGridType = openvdb::VEC_CONTRAVARIANT_ABSOLUTE;
		const FString GridName = "ClosestPoint";

		TSharedPtr<FDataflowFloatVectorVolumeImpl> FloatVectorVolumeImpl = MakeShared<FDataflowFloatVectorVolumeImpl>();

		if (this->FloatGrid->empty())
		{
			FloatVectorVolumeImpl->SetGrid(openvdb::VectorGrid::Ptr(), VectorGridType, GridName);
			return FloatVectorVolumeImpl;
		}

		FString NewGridName;
		if (OutputName == EDataflowVolumeAnalysisOutputName::AppendOperationName)
		{
			const FString CurrentGridName(this->FloatGrid->getName().c_str());
			NewGridName = CurrentGridName + "_" + GridName;
		}
		else if (OutputName == EDataflowVolumeAnalysisOutputName::CustomName)
		{
			NewGridName = CustomName;
		}

		// @todo(gmelich): Implement using interrupter

//			UE::DataflowVolumeUtils::FInterrupter NullInterrupter;

		openvdb::VectorGrid::Ptr CptVectorGrid;

		if (InMaskVolumeImpl->GetGrid())
		{
			openvdb::BoolGrid RegionMask;

			RegionMask.setTransform(this->FloatGrid->transform().copy());

			openvdb::tools::resampleToMatch<openvdb::tools::PointSampler>(*InMaskVolumeImpl->GetGrid().get(), RegionMask);

			openvdb::tools::Cpt<openvdb::FloatGrid, openvdb::BoolGrid> CptTool(*this->FloatGrid.get(), RegionMask);
			CptVectorGrid = CptTool.process(/*threaded*/true);
		}
		else
		{
			openvdb::tools::Cpt<openvdb::FloatGrid> CptTool(*this->FloatGrid.get());
			CptVectorGrid = CptTool.process(/*threaded*/true);
		}

		FloatVectorVolumeImpl->SetGrid(CptVectorGrid, VectorGridType, NewGridName);

		return FloatVectorVolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	// Functor to MultiplyAdd all active values

	struct Op {
		float Frac, Offset;
		Op(float InFrac, float InOffset) : Frac(InFrac), Offset(InOffset) {}

		inline void operator()(const openvdb::FloatGrid::ValueOnIter& iter) const
		{
			iter.setValue(*iter * Frac + Offset);
		}
	};

	static void Combine(const float InScale, const float InOffset, const openvdb::FloatGrid::Ptr& InGrid, openvdb::FloatGrid::Ptr& OutGrid)
	{
		if (FMath::IsNearlyZero(InScale - 1.f, 1e-6) && FMath::IsNearlyZero(InOffset, 1e-6))
		{
			OutGrid = InGrid->deepCopy();
		}
		else
		{
			OutGrid = openvdb::FloatGrid::create(*InGrid);

			float BackgroundValue = InScale * InGrid->background() + InOffset;
			openvdb::tools::changeBackground(OutGrid->tree(), BackgroundValue);
			openvdb::tools::foreach(OutGrid->beginValueOn(), Op(InScale, InOffset));
		}
	}
/*
	void ResampleGrids(openvdb::FloatGrid::Ptr FloatGridA, 
		openvdb::FloatGrid::Ptr FloatGridB, 
		const EDataflowVolumeSDFCombineOperation InOperation,
		const EDataflowVolumeSDFCombineResample InResample,
		const EDataflowVolumeSDFCombineInterpolation InInterpolation)
	{
		const bool NeedA = InOperation != EDataflowVolumeSDFCombineOperation::CopyB;
		const bool NeedB = InOperation != EDataflowVolumeSDFCombineOperation::CopyA && InOperation != EDataflowVolumeSDFCombineOperation::InvertA;
		const bool NeedBoth = NeedA && NeedB;
		const bool NeedLevelSets = InOperation == EDataflowVolumeSDFCombineOperation::SDFUnion || 
			InOperation == EDataflowVolumeSDFCombineOperation::SDFIntersect ||
			InOperation == EDataflowVolumeSDFCombineOperation::SDFDifference;

		int32 ResampleWhich = (int32)EDataflowVolumeSDFCombineResample::Off;

		if (!FloatGridA->empty() && !FloatGridB->empty())
		{
			if (InResample == EDataflowVolumeSDFCombineResample::HiresMatchLores || InResample == EDataflowVolumeSDFCombineResample::LoresMatchHires)
			{
				const openvdb::Vec3d aVoxSize = FloatGridA->voxelSize(), bVoxSize = FloatGridB->voxelSize();
				const double aVoxVol = aVoxSize[0] * aVoxSize[1] * aVoxSize[2], bVoxVol = bVoxSize[0] * bVoxSize[1] * bVoxSize[2];
				ResampleWhich = ((aVoxVol > bVoxVol && InResample == EDataflowVolumeSDFCombineResample::LoresMatchHires)
					|| (aVoxVol < bVoxVol && InResample == EDataflowVolumeSDFCombineResample::HiresMatchLores))
					? (int32)EDataflowVolumeSDFCombineResample::AMatchB : (int32)EDataflowVolumeSDFCombineResample::BMatchA;
			}
			else 
			{
				ResampleWhich = (int32)InResample;
			}

			openvdb::FloatGrid::Ptr aBaseGrid, bBaseGrid;

			if (FloatGridA->constTransform() != FloatGridB->constTransform()) {
				// If the A and B grid transforms don't match, one of the grids
				// should be resampled into the other's index space.
				if (InResample == EDataflowVolumeSDFCombineResample::Off) {
					if (NeedBoth) {
						// Resampling is disabled.  Just log a warning.
//						std::ostringstream ostr;
//						ostr << aGridName << " and " << bGridName << " transforms don't match";
//						self->addWarning(SOP_MESSAGE, ostr.str().c_str());
					}
				}
				else {
					if (NeedA && ResampleWhich == EDataflowVolumeSDFCombineResample::AMatchB) {
						// Resample grid A into grid B's index space.
						aBaseGrid = this->resampleToMatch(*aGrid, *bGrid, samplingOrder);
						aGrid = static_cast<const AGridT*>(aBaseGrid.get());
					}
					else if (needB && resampleWhich == RESAMPLE_B) {
						// Resample grid B into grid A's index space.
						bBaseGrid = this->resampleToMatch(*bGrid, *aGrid, samplingOrder);
						bGrid = static_cast<const BGridT*>(bBaseGrid.get());
					}
				}
			}


















		}
	}


































	//enum ResampleMode {
	//	RESAMPLE_OFF,    // don't auto-resample grids
	//	RESAMPLE_B,      // resample B to match A
	//	RESAMPLE_A,      // resample A to match B
	//	RESAMPLE_HI_RES, // resample higher-res grid to match lower-res
	//	RESAMPLE_LO_RES  // resample lower-res grid to match higher-res
	//};
	//enum { RESAMPLE_MODE_FIRST = RESAMPLE_OFF, RESAMPLE_MODE_LAST = RESAMPLE_LO_RES };


*/



	TSharedPtr<FDataflowFloatVolumeImpl> FDataflowFloatVolumeImpl::VolumeCombine(
		TSharedPtr<const FDataflowFloatVolumeImpl> InVolumeBImpl,
		const EDataflowVolumeSDFCombineOperation InOperation,
		const float InMultiplierA,
		const float InMultiplierB,
		const EDataflowVolumeSDFCombineResample InResample,
		const EDataflowVolumeSDFCombineInterpolation InInterpolation,
		const bool InPruneDegenerateTiles,
		const float InPruneTol) const
	{
		constexpr openvdb::GridClass GridClass = openvdb::GRID_UNKNOWN;

		using namespace openvdb;

		TSharedPtr<FDataflowFloatVolumeImpl> FloatVolumeImpl = MakeShared<FDataflowFloatVolumeImpl>();

		if (this->FloatGrid->empty() || InVolumeBImpl->FloatGrid->empty())
		{
			FloatVolumeImpl->SetGrid(openvdb::FloatGrid::Ptr(), GridClass, "Combine");
			return FloatVolumeImpl;
		}

		// @todo(gmelich): Implement resmapling the grids
//		ResampleGrids(this->FloatGrid, InVolumeB.FloatGrid, InOperation, InResample, InInterpolation);

		openvdb::FloatGrid::Ptr ResultGrid, TempGrid;

		switch (InOperation) {
		case EDataflowVolumeSDFCombineOperation::SDFIntersect:
			Combine(InMultiplierA, 0.f, this->FloatGrid, ResultGrid);
			Combine(InMultiplierB, 0.f, InVolumeBImpl->FloatGrid, TempGrid);
			openvdb::tools::csgIntersection(*ResultGrid, *TempGrid, /*prune*/true);

			break;

		case EDataflowVolumeSDFCombineOperation::SDFUnion:
			Combine(InMultiplierA, 0.f, this->FloatGrid, ResultGrid);
			Combine(InMultiplierB, 0.f, InVolumeBImpl->FloatGrid, TempGrid);
			openvdb::tools::csgUnion(*ResultGrid, *TempGrid, /*prune*/true);

			break;

		case EDataflowVolumeSDFCombineOperation::SDFDifference:
			Combine(InMultiplierA, 0.f, this->FloatGrid, ResultGrid);
			Combine(InMultiplierB, 0.f, InVolumeBImpl->FloatGrid, TempGrid);
			openvdb::tools::csgDifference(*ResultGrid, *TempGrid, /*prune*/true);
			
			break;

		default:
			break;
		}

		openvdb::tools::prune(ResultGrid->tree(), InPruneTol);

		FloatVolumeImpl->SetGrid(ResultGrid, openvdb::GRID_LEVEL_SET);

		return FloatVolumeImpl;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	bool FDataflowFloatVolumeImpl::CreateVolumeTexture(UVolumeTexture* InVolumeTexture) const
	{
		if (InVolumeTexture == nullptr)
		{
			return false;
		}

		const EPixelFormat PixelFormat = EPixelFormat::PF_R16F;

		if (!FloatGrid->empty())
		{
			openvdb::Coord ActiveVoxelDim = FloatGrid->evalActiveVoxelDim();
			openvdb::CoordBBox Bbox = FloatGrid->evalActiveVoxelBoundingBox();
			const int32 XMin = Bbox.min().x();
			const int32 YMin = Bbox.min().y();
			const int32 ZMin = Bbox.min().z();

			openvdb::FloatGrid::Accessor Accessor = FloatGrid->getAccessor();

			auto QueryVoxel = [&Accessor, &XMin, &YMin, &ZMin](int32 PosX, int32 PosY, int32 PosZ, void* Value)
			{
				openvdb::Coord Coord(PosX + XMin, PosY + YMin, PosZ + ZMin);
				FFloat16* const Voxel = static_cast<FFloat16*>(Value);

				if (Accessor.isValueOn(Coord))
				{
					Voxel[0].Set(Accessor.getValue(Coord));
				}
				else
				{
					Voxel[0].Set(0.f);
				}
			};

			const bool bSuccess = InVolumeTexture->UpdateSourceFromFunction(QueryVoxel, ActiveVoxelDim[0], ActiveVoxelDim[1], ActiveVoxelDim[2], TSF_R16F);

			if (!bSuccess)
			{
				return false;
			}

			return true;
		}

		return false;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowFloatVolumeImpl::VolumeToSpheresWithRelaxation(
		const int32 InPointCount,
		const int32 InScatterRandomSeed,
		const float InMinRadius,
		const float InMaxRadius,
		const int32 InRandomSeed,
		const float InIsoValue,
		const float InSpread,
		const bool InRemoveSpheres,
		const bool InApplyRelaxation,
		const int32 InSteps,
		const float InStepScalar,
		const bool InCorrectAgainstSurface,
		const float InDistanceThreshold,
		TArray<FSphere>& OutSpheres) const
	{
		if (FloatGrid)
		{
			openvdb::FloatGrid::ConstPtr GridPtr = openvdb::gridConstPtrCast<openvdb::FloatGrid>(FloatGrid);

			std::vector<openvdb::Vec4s> Spheres;

			VolumeToSpheresWithRelaxationProc(*GridPtr,
				Spheres, InPointCount, InScatterRandomSeed, InMinRadius, InMaxRadius, InRandomSeed,
				InIsoValue, InSpread, InRemoveSpheres, InApplyRelaxation, InSteps, InStepScalar, InCorrectAgainstSurface, InDistanceThreshold);

			const int32 NumSpheres = Spheres.size();
			OutSpheres.SetNum(NumSpheres);

			for (int32 Idx = 0; Idx < NumSpheres; ++Idx)
			{
				OutSpheres[Idx] = FSphere(FVector(Spheres[Idx].x(), Spheres[Idx].y(), Spheres[Idx].z()), Spheres[Idx].w());
			}
		}
	}
}

