// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowVolumeImpl.h"
#include "Dataflow/DataflowFloatVectorVolumeImpl.h"
#include "Dataflow/OpenVDB.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "MeshDescription.h"
#include "Dataflow/DataflowVolumeNodeEnums.h"

struct FDataflowBoolVolume;

namespace UE::DataflowVolume::Private
{
	/* --------------------------------------------------------------------------------------------------------------- */
	/* FDataflowFloatVolumeImpl */
	/* --------------------------------------------------------------------------------------------------------------- */

	class FDataflowVolume;
	class FDataflowIntVolumeImpl;
	class FDataflowFloatVectorVolumeImpl;
	class FDataflowBoolVolumeImpl;

	class FDataflowFloatVolumeImpl : public FDataflowVolumeImpl
	{
	private:
		openvdb::FloatGrid::Ptr FloatGrid = nullptr;

	public:
		// Virtual functions
		virtual FString VolumeInfo() const override;
		virtual void GetActiveVoxels(const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly = false) const override;
		virtual FBox GetVolumeBoundingBox() const override;
		virtual bool CreateVolumeTexture(UVolumeTexture* InVolumeTexture) const override;

		openvdb::FloatGrid::Ptr GetGrid() const;

		void SetGrid(const openvdb::FloatGrid::Ptr& InFloatGrid, const openvdb::GridClass InGridClass, const FString& InGridName = TEXT("LevelSet"));

		FString GetGridType() const;

		FString GetGridClass() const;

		static TSharedPtr<FDataflowFloatVolumeImpl> CreateSphereSDF(
			const float InVoxelSize,
			const float InRadius,
			const FVector& InCenter);

		static TSharedPtr<FDataflowFloatVolumeImpl> CreateCubeSDF(
			const float InScale, 
			const FVector& InCenter, 
			const float InVoxelSize);

		static TSharedPtr<FDataflowFloatVolumeImpl> CreatePlatonicSolidSDF(
			const int32 InFaceCount, 
			const float InScale, 
			const FVector& InCenter, 
			const float InVoxelSize);

		static TSharedPtr<FDataflowFloatVolumeImpl> CreateVolumeFromMeshDescription(
			const FMeshDescription& InMeshDescription,
			const FTransform& InTransform,
			const float InVoxelSize,
			const TSharedPtr<const FDataflowVolumeImpl>& InReferenceVolumeImpl,
			const EDataflowVolumeOutputType InOutputType,
			const FString& InGridName,
			const bool InUseWorldSpaceUnits,
			const float InExteriorBand,
			const float InInteriorBand,
			const int32 InExteriorBandVoxels,
			const int32 InInteriorBandVoxels,
			const bool bFillInterior,
			const bool bPreserveHoles,
			const float InIsoValue,
			TSharedPtr<FDataflowIntVolumeImpl>& OutFaceIndexVolume);

		void VolumeToSpheres(const int32 InMinSphereCount,
			const int32 InMaxSphereCount,
			const bool InOverlapping,
			const float InMinRadius,
			float InMaxRadius,
			const float InIsoValue,
			const int32 InInstanceCount,
			TArray<FVector>& OutSphereCenters,
			TArray<float>& OutSphereRadii) const;

		void VolumeToSpheresImproved(const int32 InMinSphereCount,
			const int32 InMaxSphereCount,
			const bool InOverlapping,
			const float InMinRadius,
			float InMaxRadius,
			const float InIsoValue,
			const int32 InInstanceCount,
			const int32 InRandomSeed,
			const EDataflowVolumeScatterType InScatterType,
			const float InSpread,
			const float InMinNumberOfPointsPerVoxel,
			const float InMaxNumberOfPointsPerVoxel,
			TArray<FVector>& OutSphereCenters,
			TArray<float>& OutSphereRadii) const;

		TSharedPtr<FDataflowFloatVolumeImpl> CovertSDFToFogVolume(
			const bool InPruneTolerance,
			const float InTolerance,
			const bool InFloodFillOutput,
			const bool InActivateInteriror) const;

		TSharedPtr<FDataflowFloatVolumeImpl> CovertFogVolumeToSDF(
			const float InFogIsoValue,
			const bool InPruneTolerance,
			const float InTolerance,
			const bool InFloodFillOutput,
			const bool InActivateInteriror) const;

		void CovertSDFToCollection(const float InIsoValue, const float InAdaptivity, FManagedArrayCollection& OutCollection) const;

		void ConvertVolumeToMeshDescription(const float InIsoValue, const float InAdaptivity, FMeshDescription& OutMeshDescription) const;

		void UniformVolumeScatter(
			const int32 InMinNumberOfPoints,
			const int32 InMaxNumberOfPoints,
			const int32 InRandomSeed,
			const float InIsoValue,
			const float InSpread,
			TArray<FVector>& OutPoints) const;

		void DenseUniformVolumeScatter(
			const float InMinNumberOfPointsPerVoxel,
			const float InMaxNumberOfPointsPerVoxel,
			const int32 InRandomSeed,
			const float InIsoValue,
			const float InSpread,
			TArray<FVector>& OutPoints) const;

		void NonUniformVolumeScatter(
			const float InMinNumberOfPointsPerVoxel,
			const float InMaxNumberOfPointsPerVoxel,
			const int32 InRandomSeed,
			const float InIsoValue,
			const float InSpread,
			TArray<FVector>& OutPoints) const;

		void VolumeSample(const TArray<FVector>& InPoints, TArray<float>& OutValues) const;

		TSharedPtr<FDataflowFloatVectorVolumeImpl> ComputeGradient(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const;
		TSharedPtr<FDataflowFloatVolumeImpl> ComputeCurvature(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const;
		TSharedPtr<FDataflowFloatVolumeImpl> ComputeLaplacian(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const;
		TSharedPtr<FDataflowFloatVectorVolumeImpl> ComputeClosestPoint(const EDataflowVolumeAnalysisOutputName OutputName, const FString& CustomName, TSharedPtr<const FDataflowBoolVolumeImpl> InMaskVolumeImpl) const;

		TSharedPtr<FDataflowFloatVolumeImpl> VolumeCombine(
			TSharedPtr<const FDataflowFloatVolumeImpl> InVolumeBImpl,
			const EDataflowVolumeSDFCombineOperation InOperation,
			const float InMultiplierA,
			const float InMultiplierB,
			const EDataflowVolumeSDFCombineResample InResample,
			const EDataflowVolumeSDFCombineInterpolation InInterpolation,
			const bool InPruneDegenerateTiles,
			const float InPruneTol) const;

		void VolumeToSpheresWithRelaxation(
			const int32 InPointCount,
			const int32 InScatterRandomSeed,
			const float InMinRadius,
			const float InMaxRadius,
			const int32 InRandomSeed,
			const float InIsoValue,
			const float InSpread,
			const bool InRemoveSpheres,
			const bool InApplyRelaxation,
			const int32 InSteps,
			const float InStepScalar,
			const bool InCorrectAgainstSurface,
			const float InDistanceThreshold,
			TArray<FSphere>& OutSpheres) const;
	};
}

