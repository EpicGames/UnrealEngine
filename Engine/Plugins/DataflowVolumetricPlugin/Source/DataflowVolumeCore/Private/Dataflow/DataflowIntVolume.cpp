// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowIntVolume.h"
#include "Dataflow/DataflowIntVolumeImpl.h"
//#include "Dataflow/OpenVDB.h"
//#include "Dataflow/DataflowVolumeUtils.h"
//#include <limits>

/* --------------------------------------------------------------------------------------------------------------- */
/* FDataflowIntVolume Implementation */
/* --------------------------------------------------------------------------------------------------------------- */

FDataflowIntVolume::FDataflowIntVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowIntVolumeImpl> InIntVolume)
	: FDataflowVolume(InIntVolume, FDataflowIntVolume::TypeName)
{
	static_assert(sizeof(FDataflowVolume) == sizeof(*this));
}

/* --------------------------------------------------------------------------------------------------------------- */

TSharedPtr<const UE::DataflowVolume::Private::FDataflowIntVolumeImpl> FDataflowIntVolume::GetIntVolumeImpl() const
{
	return StaticCastSharedPtr<const UE::DataflowVolume::Private::FDataflowIntVolumeImpl>(Volume);
}

/* --------------------------------------------------------------------------------------------------------------- */

TSharedPtr<UE::DataflowVolume::Private::FDataflowIntVolumeImpl> FDataflowIntVolume::GetIntVolumeImpl()
{
	return StaticCastSharedPtr<UE::DataflowVolume::Private::FDataflowIntVolumeImpl>(Volume);
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowIntVolume::SetIntVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowIntVolumeImpl> InIntVolume)
{
	Volume = InIntVolume;
}

/* --------------------------------------------------------------------------------------------------------------- */

FVector FDataflowIntVolume::GetVoxelSize() const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowIntVolumeImpl> IntVolume = GetIntVolumeImpl())
	{
		return IntVolume->GetVoxelSize();
	}

	return {};
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowIntVolume::VolumeSample(
	const TArray<FVector>& InPoints,
	TArray<int32>& OutValues) const
{
	using namespace UE::DataflowVolume::Private;

	if (TSharedPtr<const FDataflowIntVolumeImpl> IntVolume = GetIntVolumeImpl())
	{
		IntVolume->VolumeSample(
			InPoints,
			OutValues);
	}
}

