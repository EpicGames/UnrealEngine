// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowIntVolumeImpl.h"
#include <Chaos/ArrayND.h>
#include "Engine/VolumeTexture.h"
#include "Dataflow/DataflowVolumeUtils.h"

namespace UE::DataflowVolume::Private
{
	/* --------------------------------------------------------------------------------------------------------------- */
	/* FDataflowIntVolumeImpl */
	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowIntVolumeImpl::SetGrid(const openvdb::Int32Grid::Ptr& InIntGrid, const FString& InGridName)
	{
		FDataflowVolumeImpl::SetGrid(InIntGrid, InGridName);

		IntGrid = InIntGrid;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	openvdb::Int32Grid::Ptr FDataflowIntVolumeImpl::GetGrid() const
	{
		return IntGrid;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FString FDataflowIntVolumeImpl::GetGridType() const
	{
		if (IntGrid)
		{
			std::string GridType = IntGrid->gridType();
			return FString(GridType.c_str());
		}

		return {};
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FString FDataflowIntVolumeImpl::VolumeInfo() const
	{
		FString BaseString = FDataflowVolumeImpl::VolumeInfo();

		FString OutputStr;
		OutputStr.Appendf(TEXT("\n----------------------------------------\n"));
		OutputStr += BaseString;

		FString GridType = this->GetGridType();

		OutputStr.Appendf(TEXT("Type: %s\n"),
			*GridType);

		OutputStr.Appendf(TEXT("----------------------------------------\n"));

		return OutputStr;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowIntVolumeImpl::VolumeSample(const TArray<FVector>& InPoints, TArray<int32>& OutValues) const
	{
		using namespace openvdb;

		if (this->IntGrid->empty())
		{
			return;
		}

		openvdb::Int32Grid::ConstAccessor Accessor = this->IntGrid->getConstAccessor();

		openvdb::tools::GridSampler<openvdb::Int32Grid::ConstAccessor, openvdb::tools::BoxSampler>
			fastSampler(Accessor, this->IntGrid->transform());

		int32 Idx = 0;
		for (const FVector& Point : InPoints)
		{
			OutValues[Idx++] = fastSampler.wsSample(openvdb::Vec3R(Point.X, Point.Y, Point.Z));
		}

		return;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	void FDataflowIntVolumeImpl::GetActiveVoxels(const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly) const
	{
		if (IntGrid)
		{
			UE::DataflowVolumeUtils::GetActiveVoxels<openvdb::Int32Grid>(IntGrid, InIsovalue, OutActiveVoxels, bInteriorMaskOnly);
		}
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FBox FDataflowIntVolumeImpl::GetVolumeBoundingBox() const
	{
		return this->GetActiveVoxelBoundingBox();
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	bool FDataflowIntVolumeImpl::CreateVolumeTexture(UVolumeTexture* InVolumeTexture) const
	{
		if (InVolumeTexture == nullptr)
		{
			return false;
		}

		const EPixelFormat PixelFormat = EPixelFormat::PF_R16F;

		if (!IntGrid->empty())
		{
			openvdb::Coord ActiveVoxelDim = IntGrid->evalActiveVoxelDim();
			openvdb::CoordBBox Bbox = IntGrid->evalActiveVoxelBoundingBox();
			const int32 XMin = Bbox.min().x();
			const int32 YMin = Bbox.min().y();
			const int32 ZMin = Bbox.min().z();

			openvdb::Int32Grid::Accessor Accessor = IntGrid->getAccessor();

			auto QueryVoxel = [&Accessor, &XMin, &YMin, &ZMin](int32 PosX, int32 PosY, int32 PosZ, void* Value)
			{
				openvdb::Coord Coord(PosX + XMin, PosY + YMin, PosZ + ZMin);
				FFloat16* const Voxel = static_cast<FFloat16*>(Value);

				if (Accessor.isValueOn(Coord))
				{
					Voxel[0].Set(Accessor.getValue(Coord));
				}
				else
				{
					Voxel[0].Set(0.f);
				}
			};

			const bool bSuccess = InVolumeTexture->UpdateSourceFromFunction(QueryVoxel, ActiveVoxelDim[0], ActiveVoxelDim[1], ActiveVoxelDim[2], TSF_R16F);

			if (!bSuccess)
			{
				return false;
			}

			return true;
		}

		return false;
	}
}

