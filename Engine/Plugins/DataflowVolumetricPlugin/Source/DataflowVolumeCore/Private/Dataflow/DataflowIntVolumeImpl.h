// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowVolumeImpl.h"
#include "Dataflow/OpenVDB.h"

namespace UE::DataflowVolume::Private
{
	/* --------------------------------------------------------------------------------------------------------------- */
	/* FDataflowIntVolumeImpl */
	/* --------------------------------------------------------------------------------------------------------------- */

	class FDataflowIntVolumeImpl : public FDataflowVolumeImpl
	{
	private:
		openvdb::Int32Grid::Ptr IntGrid = nullptr;

	public:
		// Virtual functions
		virtual FString VolumeInfo() const override;
		DATAFLOWVOLUMECORE_API void GetActiveVoxels(const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly = false) const;
		virtual FBox GetVolumeBoundingBox() const override;
		virtual bool CreateVolumeTexture(UVolumeTexture* InVolumeTexture) const override;

		openvdb::Int32Grid::Ptr GetGrid() const;

		void SetGrid(const openvdb::Int32Grid::Ptr& InIntGrid, const FString& InGridName = "");

		FString GetGridType() const;

		void VolumeSample(const TArray<FVector>& InPoints, TArray<int32>& OutValues) const;

	};
}
