// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVolume.h"
#include "Dataflow/DataflowVolumeImpl.h"

/* --------------------------------------------------------------------------------------------------------------- */
/* FDataflowVolume Implementation */
/* --------------------------------------------------------------------------------------------------------------- */

FDataflowVolume::FDataflowVolume(TSharedPtr<UE::DataflowVolume::Private::FDataflowVolumeImpl> InVolume, const FName InType)
	: Volume(InVolume)
	, Type(InType)
{}

/* --------------------------------------------------------------------------------------------------------------- */

FString FDataflowVolume::VolumeInfo() const
{
	if (Volume)
	{
		return Volume->VolumeInfo();
	}

	return {};
}

/* --------------------------------------------------------------------------------------------------------------- */

int32 FDataflowVolume::GetNumActiveVoxels(const float InIsovalue, bool bInteriorMaskOnly) const
{
	if (Volume)
	{
		TArray<FBox> OutActiveVoxels;
		Volume->GetActiveVoxels(InIsovalue, OutActiveVoxels, bInteriorMaskOnly);

		return OutActiveVoxels.Num();
	}

	return 0;
}

/* --------------------------------------------------------------------------------------------------------------- */

void FDataflowVolume::GetActiveVoxels(const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly) const
{
	if (Volume)
	{
		Volume->GetActiveVoxels(InIsovalue, OutActiveVoxels, bInteriorMaskOnly);
	}
}

/* --------------------------------------------------------------------------------------------------------------- */

FBox FDataflowVolume::GetVolumeBoundingBox() const
{
	if (Volume)
	{
		return Volume->GetVolumeBoundingBox();
	}

	return {};
}

/* --------------------------------------------------------------------------------------------------------------- */

bool FDataflowVolume::CreateVolumeTexture(UVolumeTexture* InVolumeTexture) const
{
	if (Volume)
	{
		return Volume->CreateVolumeTexture(InVolumeTexture);
	}

	return false;
}

