// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/OpenVDB.h"
#include "Dataflow/DataflowVolumeNodeEnums.h"

#include "Math/RandomStream.h"

#include <tbb/blocked_range.h>
#include <tbb/parallel_for.h>
#include <tbb/parallel_reduce.h>

#include <algorithm> // for std::min(), std::max()
#include <cmath> // for std::sqrt()
#include <limits> // for std::numeric_limits
#include <memory>
#include <random>
#include <utility> // for std::pair
#include <vector>

#include <openvdb/tools/VolumeToSpheres.h>
#include <openvdb/tools/LevelSetRebuild.h>
#include <openvdb/tools/LevelSetUtil.h>
#include <openvdb/util/NullInterrupter.h>

// Ideas:
// 
// - expose random seed as input
// - expose scatter type as input
// - make sure that the computed radius for the scattered points never gets bigger than maxRadius
// - move scattered points with phi < phi_threshold in the inverse gradient direction by a random amount
// - option to don't clamp radius by closestpointonsurface
//

template<typename GridT, typename InterrupterT = openvdb::util::NullInterrupter>
void FillWithSpheres(
	const GridT& grid,
	std::vector<openvdb::Vec4s>& spheres,
	const openvdb::Vec2i& sphereCount,
	bool overlapping,
	float minRadius,
	float maxRadius,
	float isovalue,
	int instanceCount,
	int32 InRandomSeed,
	EDataflowVolumeScatterType InScatterType,
	float InSpread,
	float InMinNumberOfPointsPerVoxel,
	float InMaxNumberOfPointsPerVoxel,
	InterrupterT* interrupter = nullptr)
{
	spheres.clear();

	if (grid.empty()) return;

	const int
		minSphereCount = sphereCount[0],
		maxSphereCount = sphereCount[1];
	if ((minSphereCount > maxSphereCount) || (maxSphereCount < 1)) {
		OPENVDB_LOG_WARN("fillWithSpheres: minimum sphere count ("
			<< minSphereCount << ") exceeds maximum count (" << maxSphereCount << ")");
		return;
	}
	spheres.reserve(maxSphereCount);

	auto gridPtr = grid.copy(); // shallow copy

	if (gridPtr->getGridClass() == openvdb::GRID_LEVEL_SET) {
		// Clamp the isovalue to the level set's background value minus epsilon.
		// (In a valid narrow-band level set, all voxels, including background voxels,
		// have values less than or equal to the background value, so an isovalue
		// greater than or equal to the background value would produce a mask with
		// effectively infinite extent.)
		isovalue = std::min(isovalue,
			static_cast<float>(gridPtr->background() - openvdb::math::Tolerance<float>::value()));
	}
	else if (gridPtr->getGridClass() == openvdb::GRID_FOG_VOLUME) {
		// Clamp the isovalue of a fog volume between epsilon and one,
		// again to avoid a mask with infinite extent.  (Recall that
		// fog volume voxel values vary from zero outside to one inside.)
		isovalue = openvdb::math::Clamp(isovalue, openvdb::math::Tolerance<float>::value(), 1.f);
	}

	// ClosestSurfacePoint is inaccurate for small grids.
	// Resample the input grid if it is too small.
	auto numVoxels = gridPtr->activeVoxelCount();
	if (numVoxels < 10000) {
		const auto scale = 1.0 / openvdb::math::Cbrt(2.0 * 10000.0 / double(numVoxels));
		auto scaledXform = gridPtr->transform().copy();
		scaledXform->preScale(scale);

		auto newGridPtr = openvdb::tools::levelSetRebuild(*gridPtr, isovalue,
			openvdb::LEVEL_SET_HALF_WIDTH, openvdb::LEVEL_SET_HALF_WIDTH, scaledXform.get(), interrupter);

		const auto newNumVoxels = newGridPtr->activeVoxelCount();
		if (newNumVoxels > numVoxels) {
			OPENVDB_LOG_DEBUG_RUNTIME("fillWithSpheres: resampled input grid from "
				<< numVoxels << " voxel" << (numVoxels == 1 ? "" : "s")
				<< " to " << newNumVoxels << " voxel" << (newNumVoxels == 1 ? "" : "s"));
			gridPtr = newGridPtr;
			numVoxels = newNumVoxels;
		}
	}

	const bool addNarrowBandPoints = (numVoxels < 10000);
	int instances = std::max(instanceCount, maxSphereCount);

	using TreeT = typename GridT::TreeType;
	using BoolTreeT = typename TreeT::template ValueConverter<bool>::Type;
	using Int16TreeT = typename TreeT::template ValueConverter<openvdb::Int16>::Type;

	using RandGen = std::mersenne_twister_engine<uint32_t, 32, 351, 175, 19,
		0xccab8ee7, 11, 0xffffffff, 7, 0x31b6ab00, 15, 0xffe50000, 17, 1812433253>; // mt11213b
	RandGen mtRand(InRandomSeed);

	const TreeT& tree = gridPtr->tree();
	openvdb::math::Transform transform = gridPtr->transform();

	std::vector<openvdb::Vec3R> instancePoints;
	{
		// Compute a mask of the voxels enclosed by the isosurface.
		typename openvdb::Grid<BoolTreeT>::Ptr interiorMaskPtr;
		if (gridPtr->getGridClass() == openvdb::GRID_LEVEL_SET) {
			interiorMaskPtr = openvdb::tools::sdfInteriorMask(*gridPtr, isovalue);
		}
		else {
			// For non-level-set grids, the interior mask comprises the active voxels.
			interiorMaskPtr = typename openvdb::Grid<BoolTreeT>::Ptr(openvdb::Grid<BoolTreeT>::create(false));
			interiorMaskPtr->setTransform(transform.copy());
			interiorMaskPtr->tree().topologyUnion(tree);
		}

		if (interrupter && interrupter->wasInterrupted()) return;

		// If the interior mask is small and eroding it results in an empty grid,
		// use the uneroded mask instead.  (But if the minimum sphere count is zero,
		// then eroding away the mask is acceptable.)
		if (!addNarrowBandPoints || (minSphereCount <= 0)) {
			openvdb::tools::erodeActiveValues(interiorMaskPtr->tree(), 1, openvdb::tools::NN_FACE, openvdb::tools::IGNORE_TILES);
			openvdb::tools::pruneInactive(interiorMaskPtr->tree());
		}
		else {
			auto& maskTree = interiorMaskPtr->tree();
			auto copyOfTree = openvdb::StaticPtrCast<BoolTreeT>(maskTree.copy());
			openvdb::tools::erodeActiveValues(maskTree, 1, openvdb::tools::NN_FACE, openvdb::tools::IGNORE_TILES);
			openvdb::tools::pruneInactive(maskTree);
			if (maskTree.empty()) { interiorMaskPtr->setTree(copyOfTree); }
		}

		// Scatter candidate sphere centroids (instancePoints)
		instancePoints.reserve(instances);
		openvdb::tools::v2s_internal::PointAccessor ptnAcc(instancePoints);

		const auto scatterCount = openvdb::Index64(addNarrowBandPoints ? (instances / 2) : instances);

		FRandomStream RandStream(InRandomSeed);
		float NumPointsPerVoxel = InMinNumberOfPointsPerVoxel;
		if (InMaxNumberOfPointsPerVoxel > InMinNumberOfPointsPerVoxel)
		{
			NumPointsPerVoxel = RandStream.FRandRange(InMinNumberOfPointsPerVoxel, InMaxNumberOfPointsPerVoxel);
		}

		if (InScatterType == EDataflowVolumeScatterType::Uniform)
		{
			openvdb::tools::UniformPointScatter<openvdb::tools::v2s_internal::PointAccessor, RandGen, InterrupterT> UniformScatter(
				ptnAcc, scatterCount, mtRand, double(InSpread), interrupter);
			UniformScatter(*interiorMaskPtr);
		}
		else if (InScatterType == EDataflowVolumeScatterType::DenseUniform)
		{
			openvdb::tools::DenseUniformPointScatter<openvdb::tools::v2s_internal::PointAccessor, RandGen, InterrupterT> DenseUniformScatter(
				ptnAcc, NumPointsPerVoxel, mtRand, double(InSpread), interrupter);
			DenseUniformScatter(*interiorMaskPtr);
		}
		else if (InScatterType == EDataflowVolumeScatterType::NonUniform)
		{
			openvdb::tools::NonUniformPointScatter<openvdb::tools::v2s_internal::PointAccessor, RandGen, InterrupterT> NonUniformScatter(
				ptnAcc, NumPointsPerVoxel, mtRand, double(InSpread), interrupter);
			NonUniformScatter(*interiorMaskPtr);
		}
	}

	if (interrupter && interrupter->wasInterrupted()) return;

	auto csp = openvdb::tools::ClosestSurfacePoint<GridT>::create(*gridPtr, isovalue, interrupter);
	if (!csp) return;

	// Add extra instance points in the interior narrow band.
	if (instancePoints.size() < size_t(instances)) {
		const Int16TreeT& signTree = csp->signTree();
		for (auto leafIt = signTree.cbeginLeaf(); leafIt; ++leafIt) {
			for (auto it = leafIt->cbeginValueOn(); it; ++it) {
				const int flags = int(it.getValue());
				if (!(openvdb::tools::volume_to_mesh_internal::EDGES & flags)
					&& (openvdb::tools::volume_to_mesh_internal::INSIDE & flags))
				{
					instancePoints.push_back(transform.indexToWorld(it.getCoord()));
				}
				if (instancePoints.size() == size_t(instances)) break;
			}
			if (instancePoints.size() == size_t(instances)) break;
		}
	}

	if (interrupter && interrupter->wasInterrupted()) return;

	// Assign a radius to each candidate sphere.  The radius is the world-space
	// distance from the sphere's center to the closest surface point.
	std::vector<float> instanceRadius;
	if (!csp->search(instancePoints, instanceRadius)) return;

	minRadius = float(minRadius * transform.voxelSize()[0]);
	maxRadius = float(maxRadius * transform.voxelSize()[0]);

	float largestRadius = 0.0;
	int largestRadiusIdx = 0;
	for (size_t n = 0, N = instancePoints.size(); n < N; ++n) 
	{
		if (instanceRadius[n] > maxRadius)
		{
			instanceRadius[n] = maxRadius;
		}

		if (instanceRadius[n] > largestRadius) 
		{
			largestRadius = instanceRadius[n];
			largestRadiusIdx = int(n);
		}
	}

	std::vector<unsigned char> instanceMask(instancePoints.size(), 0);


	for (size_t s = 0, S = std::min(size_t(maxSphereCount), instancePoints.size()); s < S; ++s) {

		if (interrupter && interrupter->wasInterrupted()) return;

		largestRadius = std::min(maxRadius, largestRadius);

		if ((int(s) >= minSphereCount) && (largestRadius < minRadius)) break;

		const openvdb::Vec4s sphere(
			float(instancePoints[largestRadiusIdx].x()),
			float(instancePoints[largestRadiusIdx].y()),
			float(instancePoints[largestRadiusIdx].z()),
			largestRadius);

		spheres.push_back(sphere);
		instanceMask[largestRadiusIdx] = 1;

		openvdb::tools::v2s_internal::UpdatePoints op(
			sphere, instancePoints, instanceRadius, instanceMask, overlapping);
		op.run();

		largestRadius = op.radius();
		largestRadiusIdx = op.index();
	}
} // fillWithSpheres

/* --------------------------------------------------------------------------------------------------------------- */

namespace DataflowVolumeToSpheres::Private
{
	static FVector ToVector(const openvdb::Vec3R& InPoint)
	{
		return FVector(InPoint.x(), InPoint.y(), InPoint.z());
	}

	static openvdb::Vec3R ToPoint(const FVector& InVector)
	{
		return openvdb::Vec3R(InVector.X, InVector.Y, InVector.Z);
	}

	static bool IsSphereContainedInSphere(const FVector& InCenterA,
		const FVector& InCenterB,
		const float InRadiusA,
		const float InRadiusB)
	{
		if (InRadiusA > InRadiusB)
		{
			float Distance = (InCenterA - InCenterB).Length();

			if (Distance <= (InRadiusA - InRadiusB))
			{
				return true;
			}
		}

		return false;
	}

	static void ComputeDisplacement(const FVector& InCenterA,
		const FVector& InCenterB,
		const float InRadiusA,
		const float InRadiusB,
		const float InStepScalar,
		FVector& OutDisplacementA,
		FVector& OutDisplacementB)
	{
		OutDisplacementA.Set(0.f, 0.f, 0.f);
		OutDisplacementB.Set(0.f, 0.f, 0.f);

		FVector Diff = InCenterB - InCenterA;
		float Distance = Diff.Length();
		
		if (Distance > UE_SMALL_NUMBER)
		{
			if (Distance <= (InRadiusA + InRadiusB) && InRadiusA > UE_SMALL_NUMBER && InRadiusB > UE_SMALL_NUMBER)
			{
				float Delta = InRadiusA + InRadiusB - Distance;

				Diff.Normalize();

				OutDisplacementA = -Diff * InStepScalar * Delta * InRadiusB / (InRadiusA + InRadiusB);
				OutDisplacementB = Diff * InStepScalar * Delta * InRadiusA / (InRadiusA + InRadiusB);
			}
		}
	}

	static bool ComputeDisplacementAgainsSurface(const FVector& InCenter,
		const float InRadius,
		const float InDistanceThreshold,
		const auto& InCsp,
		const openvdb::FloatGrid::ConstAccessor& InAccessor,
		const auto& InTransform,
		FVector& OutDisplacement)
	{
		OutDisplacement.Set(0.f, 0.f, 0.f);

		// Get closest point on the isosurface to InCenterA and InCenterB
		std::vector<openvdb::Vec3R> Points;

		Points.push_back(DataflowVolumeToSpheres::Private::ToPoint(InCenter));

		std::vector<float> Distances;

		if (!InCsp->searchAndReplace(Points, Distances))
		{
			return false;
		}

		FVector ClosestPoint = ToVector(Points[0]);

		// Get phi values for InCenterA and InCenterB
		openvdb::tools::GridSampler<openvdb::FloatGrid::ConstAccessor, openvdb::tools::BoxSampler>
			fastSampler(InAccessor, InTransform);

		float Phi = fastSampler.wsSample(DataflowVolumeToSpheres::Private::ToPoint(InCenter));

		FVector Gradient = ClosestPoint - InCenter;

		// If InCenter is on the isosurface then gradient is nonspecific and we can't displace
		if (Gradient.Length() < UE_SMALL_NUMBER)
		{
			return true;
		}

		// Center is inside of isosurface
		if (Phi < 0.f)
		{
			if (Gradient.Length() < InRadius && Gradient.Length() > UE_SMALL_NUMBER)
			{
				float Delta = InRadius - Gradient.Length();
				if (Delta > InDistanceThreshold)
				{
					Gradient.Normalize();
					OutDisplacement = -Gradient * (Delta - InDistanceThreshold);
				}
			}
		}
		else
		{
			float Delta = Gradient.Length() + InRadius - InDistanceThreshold;

			Gradient.Normalize();
			OutDisplacement = Gradient * Delta;
		}

		return true;
	}
}

template<typename GridT, typename InterrupterT = openvdb::util::NullInterrupter>
void VolumeToSpheresWithRelaxationProc(
	const GridT& InGrid,
	std::vector<openvdb::Vec4s>& OutSpheres,
	int32 InPointCount,
	const int32 InScatterRandomSeed,
	const float InMinRadius,
	const float InMaxRadius,
	const int32 InRandomSeed,
	float InIsovalue,
	float InSpread,
	bool InRemoveSpheres,
	bool InApplyRelaxation,
	int32 InSteps,
	float InStepScalar,
	bool InCorrectAgainstSurface,
	float InDistanceThreshold,
	InterrupterT* InInterrupter = nullptr)
{
	OutSpheres.clear();

	if (InGrid.empty()) return;

	auto gridPtr = InGrid.copy(); // shallow copy

	if (gridPtr->getGridClass() == openvdb::GRID_LEVEL_SET) {
		// Clamp the isovalue to the level set's background value minus epsilon.
		// (In a valid narrow-band level set, all voxels, including background voxels,
		// have values less than or equal to the background value, so an isovalue
		// greater than or equal to the background value would produce a mask with
		// effectively infinite extent.)
		InIsovalue = std::min(InIsovalue,
			static_cast<float>(gridPtr->background() - openvdb::math::Tolerance<float>::value()));
	}
	else if (gridPtr->getGridClass() == openvdb::GRID_FOG_VOLUME) {
		// Clamp the isovalue of a fog volume between epsilon and one,
		// again to avoid a mask with infinite extent.  (Recall that
		// fog volume voxel values vary from zero outside to one inside.)
		InIsovalue = openvdb::math::Clamp(InIsovalue, openvdb::math::Tolerance<float>::value(), 1.f);
	}

	// ClosestSurfacePoint is inaccurate for small grids.
	// Resample the input grid if it is too small.
	auto numVoxels = gridPtr->activeVoxelCount();

	if (!numVoxels) return;

	if (numVoxels < 10000) {
		const auto scale = 1.0 / openvdb::math::Cbrt(2.0 * 10000.0 / double(numVoxels));
		auto scaledXform = gridPtr->transform().copy();
		scaledXform->preScale(scale);

		auto newGridPtr = openvdb::tools::levelSetRebuild(*gridPtr, InIsovalue,
			openvdb::LEVEL_SET_HALF_WIDTH, openvdb::LEVEL_SET_HALF_WIDTH, scaledXform.get(), InInterrupter);

		const auto newNumVoxels = newGridPtr->activeVoxelCount();
		if (newNumVoxels > numVoxels) {
			OPENVDB_LOG_DEBUG_RUNTIME("fillWithSpheres: resampled input grid from "
				<< numVoxels << " voxel" << (numVoxels == 1 ? "" : "s")
				<< " to " << newNumVoxels << " voxel" << (newNumVoxels == 1 ? "" : "s"));
			gridPtr = newGridPtr;
			numVoxels = newNumVoxels;
		}
	}

	const bool addNarrowBandPoints = (numVoxels < 10000);
	int instances = InPointCount;

	using TreeT = typename GridT::TreeType;
	using BoolTreeT = typename TreeT::template ValueConverter<bool>::Type;
	using Int16TreeT = typename TreeT::template ValueConverter<openvdb::Int16>::Type;

	using RandGen = std::mersenne_twister_engine<uint32_t, 32, 351, 175, 19,
		0xccab8ee7, 11, 0xffffffff, 7, 0x31b6ab00, 15, 0xffe50000, 17, 1812433253>; // mt11213b
	RandGen mtRand(InScatterRandomSeed);

	const TreeT& tree = gridPtr->tree();
	openvdb::math::Transform transform = gridPtr->transform();

	std::vector<openvdb::Vec3R> instancePoints;
	{
		// Compute a mask of the voxels enclosed by the isosurface.
		typename openvdb::Grid<BoolTreeT>::Ptr interiorMaskPtr;
		if (gridPtr->getGridClass() == openvdb::GRID_LEVEL_SET) {
			interiorMaskPtr = openvdb::tools::sdfInteriorMask(*gridPtr, InIsovalue);
		}
		else {
			// For non-level-set grids, the interior mask comprises the active voxels.
			interiorMaskPtr = typename openvdb::Grid<BoolTreeT>::Ptr(openvdb::Grid<BoolTreeT>::create(false));
			interiorMaskPtr->setTransform(transform.copy());
			interiorMaskPtr->tree().topologyUnion(tree);
		}

		if (InInterrupter && InInterrupter->wasInterrupted()) return;

		// If the interior mask is small and eroding it results in an empty grid,
		// use the uneroded mask instead.  (But if the minimum sphere count is zero,
		// then eroding away the mask is acceptable.)
		if (!addNarrowBandPoints) {
			openvdb::tools::erodeActiveValues(interiorMaskPtr->tree(), 1, openvdb::tools::NN_FACE, openvdb::tools::IGNORE_TILES);
			openvdb::tools::pruneInactive(interiorMaskPtr->tree());
		}
		else {
			auto& maskTree = interiorMaskPtr->tree();
			auto copyOfTree = openvdb::StaticPtrCast<BoolTreeT>(maskTree.copy());
			openvdb::tools::erodeActiveValues(maskTree, 1, openvdb::tools::NN_FACE, openvdb::tools::IGNORE_TILES);
			openvdb::tools::pruneInactive(maskTree);
			if (maskTree.empty()) { interiorMaskPtr->setTree(copyOfTree); }
		}

		// Scatter candidate sphere centroids (instancePoints)
		instancePoints.reserve(instances);
		openvdb::tools::v2s_internal::PointAccessor ptnAcc(instancePoints);

		const auto scatterCount = openvdb::Index64(InPointCount);

		openvdb::tools::UniformPointScatter<openvdb::tools::v2s_internal::PointAccessor, RandGen, InterrupterT> UniformScatter(
				ptnAcc, scatterCount, mtRand, double(InSpread), InInterrupter);
			UniformScatter(*interiorMaskPtr);

		InPointCount = UniformScatter.getPointCount();
	}

	if (InInterrupter && InInterrupter->wasInterrupted()) return;

	// Random radius
	TArray<float> InstanceRadius;
	InstanceRadius.SetNumUninitialized(InPointCount);

	FRandomStream RandStream(InRandomSeed);

	for (int32 Idx = 0; Idx < InPointCount; ++Idx)
	{
		InstanceRadius[Idx] = RandStream.FRandRange(InMinRadius, InMaxRadius);
	}

	// Remove spheres
	TArray<bool> ValidSphere;
	ValidSphere.Init(true, InPointCount);

	if (InRemoveSpheres)
	{
		for (int32 Idx = 0; Idx < InPointCount; ++Idx)
		{
			if (!ValidSphere[Idx]) continue;

			for (int32 IdxOther = Idx + 1; IdxOther < InPointCount; ++IdxOther)
			{
				if (!ValidSphere[IdxOther]) continue;

				if (DataflowVolumeToSpheres::Private::IsSphereContainedInSphere(DataflowVolumeToSpheres::Private::ToVector(instancePoints[Idx]),
					DataflowVolumeToSpheres::Private::ToVector(instancePoints[IdxOther]),
					InstanceRadius[Idx],
					InstanceRadius[IdxOther]))
				{
					ValidSphere[IdxOther] = false;
				}
			}
		}
	}

	// Relaxation
	if (InApplyRelaxation)
	{
		// For closestPoint query
		auto csp = openvdb::tools::ClosestSurfacePoint<GridT>::create(*gridPtr, InIsovalue, InInterrupter);
		if (!csp) return;

		// For Phi value query
		openvdb::FloatGrid::ConstAccessor Accessor = gridPtr->getConstAccessor();

		for (int32 Idx_Step = 0; Idx_Step < InSteps; ++Idx_Step)
		{
			for (int32 Idx = 0; Idx < InPointCount; ++Idx)
			{
				if (!ValidSphere[Idx]) continue;

				for (int32 IdxOther = Idx + 1; IdxOther < InPointCount; ++IdxOther)
				{
					if (!ValidSphere[IdxOther]) continue;

					// Compute displacement
					FVector DisplacementA, DisplacementB;

					DataflowVolumeToSpheres::Private::ComputeDisplacement(DataflowVolumeToSpheres::Private::ToVector(instancePoints[Idx]),
						DataflowVolumeToSpheres::Private::ToVector(instancePoints[IdxOther]),
						InstanceRadius[Idx],
						InstanceRadius[IdxOther],
						InStepScalar,
						DisplacementA,
						DisplacementB);

					// Add displacement
					instancePoints[Idx].x() += DisplacementA.X;
					instancePoints[Idx].y() += DisplacementA.Y;
					instancePoints[Idx].z() += DisplacementA.Z;

					instancePoints[IdxOther].x() += DisplacementB.X;
					instancePoints[IdxOther].y() += DisplacementB.Y;
					instancePoints[IdxOther].z() += DisplacementB.Z;

					if (InCorrectAgainstSurface)
					{
						// Check against surface
						FVector Displacement;

						if (DataflowVolumeToSpheres::Private::ComputeDisplacementAgainsSurface(DataflowVolumeToSpheres::Private::ToVector(instancePoints[Idx]),
							InstanceRadius[Idx],
							InDistanceThreshold,
							csp,
							Accessor,
							gridPtr->transform(),
							Displacement))
						{
							// Add displacement
							instancePoints[Idx].x() += Displacement.X;
							instancePoints[Idx].y() += Displacement.Y;
							instancePoints[Idx].z() += Displacement.Z;
						}

						if (DataflowVolumeToSpheres::Private::ComputeDisplacementAgainsSurface(DataflowVolumeToSpheres::Private::ToVector(instancePoints[IdxOther]),
							InstanceRadius[IdxOther],
							InDistanceThreshold,
							csp,
							Accessor,
							gridPtr->transform(),
							Displacement))

						{
							// Add displacement
							instancePoints[IdxOther].x() += Displacement.X;
							instancePoints[IdxOther].y() += Displacement.Y;
							instancePoints[IdxOther].z() += Displacement.Z;
						}
					}
				}
			}
		}
	}

	// Create spheres
	int32 NumValidSpheres = 0;

	for (int32 Idx = 0; Idx < InPointCount; ++Idx)
	{
		if (ValidSphere[Idx])
		{
			NumValidSpheres++;
		}
	}

	OutSpheres.reserve(NumValidSpheres);

	for (int32 Idx = 0; Idx < InPointCount; ++Idx)
	{
		if (ValidSphere[Idx])
		{
			const openvdb::Vec4s sphere(
				float(instancePoints[Idx].x()),
				float(instancePoints[Idx].y()),
				float(instancePoints[Idx].z()),
				InstanceRadius[Idx]);

			OutSpheres.push_back(sphere);
		}
	}
}

