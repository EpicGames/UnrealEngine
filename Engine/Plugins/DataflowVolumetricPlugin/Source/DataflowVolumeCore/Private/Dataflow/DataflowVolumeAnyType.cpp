// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVolumeAnyType.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"

namespace UE::Dataflow
{
	void RegisterVolumeAnyTypes()
	{
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowVolumeTypes);
	}
}