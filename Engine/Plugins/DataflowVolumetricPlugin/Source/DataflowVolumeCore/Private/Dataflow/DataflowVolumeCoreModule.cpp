// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVolumeCoreModule.h"

#include "CoreMinimal.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Dataflow/OpenVDB.h"

#define LOCTEXT_NAMESPACE "DataflowCore"

void IDataflowVolumeCoreModule::StartupModule()
{
	// Initialize OpenVDB
	openvdb::initialize();
}


void IDataflowVolumeCoreModule::ShutdownModule()
{}

IMPLEMENT_MODULE(IDataflowVolumeCoreModule, DataflowVolumeCore)

#undef LOCTEXT_NAMESPACE
