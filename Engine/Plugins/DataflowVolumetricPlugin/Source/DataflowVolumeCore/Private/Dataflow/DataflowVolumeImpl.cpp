// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVolumeImpl.h"
#include "Dataflow/OpenVDB.h"

namespace UE::DataflowVolume::Private
{
	/* --------------------------------------------------------------------------------------------------------------- */
	/* FDataflowVolumeImpl */
	/* --------------------------------------------------------------------------------------------------------------- */

	bool FDataflowVolumeImpl::IsGridEmpty() const
	{
		if (BaseGrid)
		{
			return BaseGrid->empty();
		}

		return false;
	}

	FVector FDataflowVolumeImpl::GetVoxelSize() const
	{
		if (BaseGrid)
		{
			openvdb::Vec3d VoxelSize = BaseGrid->voxelSize();
			return FVector(VoxelSize.x(), VoxelSize.y(), VoxelSize.z());
		}
		
		return FVector(0);
	}

	FString FDataflowVolumeImpl::GetGridName() const
	{
		if (BaseGrid)
		{
			std::string GridName = BaseGrid->getName();
			return FString(GridName.c_str());
		}
		
		return {};
	}

	int32 FDataflowVolumeImpl::GetActiveVoxelCount() const
	{
		if (BaseGrid)
		{
			openvdb::Index64 ActiveVoxelCount = BaseGrid->activeVoxelCount();
			return (int32)ActiveVoxelCount;
		}

		return 0;
	}

	FIntVector3 FDataflowVolumeImpl::GetActiveVoxelDim() const
	{
		if (BaseGrid)
		{
			return FIntVector3(
				BaseGrid->evalActiveVoxelDim().x(),
				BaseGrid->evalActiveVoxelDim().y(),
				BaseGrid->evalActiveVoxelDim().z());
		}

		return {};
	}

	FBox FDataflowVolumeImpl::GetActiveVoxelBoundingBox() const
	{
		if (BaseGrid)
		{
			openvdb::CoordBBox BBox = BaseGrid->evalActiveVoxelBoundingBox();
			openvdb::BBoxd BBoxInWorldSpace = BaseGrid->transform().indexToWorld(BBox);

			FVector Min(BBoxInWorldSpace.min().x(), BBoxInWorldSpace.min().y(), BBoxInWorldSpace.min().z());
			FVector Max(BBoxInWorldSpace.max().x(), BBoxInWorldSpace.max().y(), BBoxInWorldSpace.max().z());

			return FBox(Min, Max);
		}

		return {};
	}

	FTransform FDataflowVolumeImpl::GetGridTransform() const
	{
		if (BaseGrid)
		{
			openvdb::math::Transform GridTransform = BaseGrid->transform();
			openvdb::Mat4R xform = GridTransform.baseMap()->getAffineMap()->getMat4();

			FMatrix Matrix = FMatrix::Identity;

			for (int32 IdxRow = 0; IdxRow < 4; ++IdxRow)
			{
				for (int32 IdxCol = 0; IdxCol < 4; ++IdxCol)
				{
					Matrix.M[IdxCol][IdxRow] = xform[IdxCol][IdxRow];
				}
			}

			FTransform Transform(Matrix);
			return Transform;
		}

		return {};
	}

	FString FDataflowVolumeImpl::VolumeInfo() const
	{
		FString GridName = this->GetGridName();
		FIntVector3 GridDimensions = this->GetActiveVoxelDim();
		int32 ActiveVoxels = this->GetActiveVoxelCount();
		FVector VoxelSize = this->GetVoxelSize();
		FBox BoundingBox = this->GetActiveVoxelBoundingBox();
		FTransform Transform = this->GetGridTransform();

		FString OutputStr;
		OutputStr.Appendf(TEXT("Grid Name: %s\nDimensions: [%d,%d,%d]\nActive Voxels: %s\nVoxel Size: %.3f\nBounding Box: [%3.2f,%3.2f,%3.2f][%3.2f,%3.2f,%3.2f]\nTransform: %3.2f,%3.2f,%3.2f|%3.2f,%3.2f,%3.2f|%3.2f,%3.2f,%3.2f\n"),
			*GridName,
			GridDimensions.X, GridDimensions.Y, GridDimensions.Z,
			*FString::FormatAsNumber(ActiveVoxels),
			VoxelSize.X,
			BoundingBox.Min.X, BoundingBox.Min.Y, BoundingBox.Min.Z,
			BoundingBox.Max.X, BoundingBox.Max.Y, BoundingBox.Max.Z,
			Transform.GetTranslation().X, Transform.GetTranslation().Y, Transform.GetTranslation().Z,
			Transform.GetRotation().Euler().X, Transform.GetRotation().Euler().Y, Transform.GetRotation().Euler().Z,
			Transform.GetScale3D().X, Transform.GetScale3D().Y, Transform.GetScale3D().Z);

		return OutputStr;
	}
	
	void FDataflowVolumeImpl::SetGrid(const openvdb::GridBase::Ptr& InGrid, const FString& InGridName)
	{
		BaseGrid = InGrid;

		std::string GridNameString(TCHAR_TO_UTF8(*InGridName));
		BaseGrid->setName(GridNameString);

		BaseGrid->setCreator("UE5_Dataflow");
	}
}