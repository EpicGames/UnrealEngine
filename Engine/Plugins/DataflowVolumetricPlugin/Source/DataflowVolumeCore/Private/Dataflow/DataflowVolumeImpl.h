// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/OpenVDB.h"
#include "Math/UnrealMath.h"
#include "Containers/UnrealString.h"

class UVolumeTexture;

namespace UE::DataflowVolume::Private
{
	/* --------------------------------------------------------------------------------------------------------------- */
	/* FDataflowVolumeImpl */
	/* --------------------------------------------------------------------------------------------------------------- */
	class FDataflowVolumeImpl
	{
	private:
		openvdb::GridBase::Ptr BaseGrid;

	public:
		bool IsGridEmpty() const;

		FVector GetVoxelSize() const;

		FString GetGridName() const;

		int32 GetActiveVoxelCount() const;

		virtual void GetActiveVoxels(const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly = false) const = 0;

		virtual FBox GetVolumeBoundingBox() const = 0;

		FIntVector3 GetActiveVoxelDim() const;

		FBox GetActiveVoxelBoundingBox() const;

		FTransform GetGridTransform() const;

		virtual FString VolumeInfo() const;

		virtual ~FDataflowVolumeImpl() {}

		virtual bool CreateVolumeTexture(UVolumeTexture* InVolumeTexture) const = 0;

	protected:
		void SetGrid(const openvdb::GridBase::Ptr& InGrid, const FString& InGridName);
	};
}