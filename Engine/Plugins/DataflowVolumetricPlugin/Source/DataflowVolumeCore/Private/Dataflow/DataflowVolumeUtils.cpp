// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVolumeUtils.h"
#include "Dataflow/OpenVDB.h"
#include "StaticMeshOperations.h"
#include "StaticMeshAttributes.h"
#include "RenderMath.h"

namespace UE::DataflowVolumeUtils
{
	FMeshDescriptionAdapter::FMeshDescriptionAdapter(const FMeshDescription& InRawMesh, const openvdb::math::Transform& InTransform) :
		RawMesh(&InRawMesh), Transform(InTransform)
	{
		InitializeCacheData();
	}

	FMeshDescriptionAdapter::FMeshDescriptionAdapter(const FMeshDescriptionAdapter& other)
		: RawMesh(other.RawMesh), Transform(other.Transform)
	{
		InitializeCacheData();
	}

	void FMeshDescriptionAdapter::InitializeCacheData()
	{
		VertexPositions = RawMesh->GetVertexPositions();
		TriangleCount = RawMesh->Triangles().Num();

		IndexBuffer.Reserve(TriangleCount * 3);

		for (const FTriangleID TriangleID : RawMesh->Triangles().GetElementIDs())
		{
			IndexBuffer.Add(RawMesh->GetTriangleVertexInstance(TriangleID, 0));
			IndexBuffer.Add(RawMesh->GetTriangleVertexInstance(TriangleID, 1));
			IndexBuffer.Add(RawMesh->GetTriangleVertexInstance(TriangleID, 2));
		}


	}

	size_t FMeshDescriptionAdapter::polygonCount() const
	{
		return size_t(TriangleCount);
	}

	size_t FMeshDescriptionAdapter::pointCount() const
	{
		return size_t(RawMesh->Vertices().Num());
	}

	void FMeshDescriptionAdapter::getIndexSpacePoint(size_t FaceNumber, size_t CornerNumber, openvdb::Vec3d& pos) const
	{
		// Get the vertex position in local space.
		const FVertexInstanceID VertexInstanceID = IndexBuffer[FaceNumber * 3 + CornerNumber];
		// float3 position 
		const FVertexID VertexID = RawMesh->GetVertexInstanceVertex(VertexInstanceID);


		FVector3f Position = VertexPositions[VertexID];
		pos = Transform.worldToIndex(openvdb::Vec3d(Position.X, Position.Y, Position.Z));
	};

	static void ResetStaticMeshDescription(FMeshDescription& MeshDecription)
	{
		MeshDecription.Empty();
		FStaticMeshAttributes Attributes(MeshDecription);
		Attributes.Register();
	}

	void AOSMeshToRawMesh(const FAOSMesh& AOSMesh, FMeshDescription& OutRawMesh)
	{

		ResetStaticMeshDescription(OutRawMesh);

		FStaticMeshAttributes Attributes(OutRawMesh);
		TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
		TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
		TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
		TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

		const uint32 DstNumPositions = AOSMesh.GetNumVertexes();
		const uint32 DstNumIndexes = AOSMesh.GetNumIndexes();

		if (VertexInstanceUVs.GetNumChannels() < 1)
		{
			VertexInstanceUVs.SetNumChannels(1);
		}

		FPolygonGroupID PolygonGroupID = INDEX_NONE;
		if (OutRawMesh.PolygonGroups().Num() == 0)
		{
			PolygonGroupID = OutRawMesh.CreatePolygonGroup();
			PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = FName(*FString::Printf(TEXT("ProxyLOD_Material_%d"), FMath::Rand()));
		}
		else
		{
			PolygonGroupID = OutRawMesh.PolygonGroups().GetFirstValidID();
		}

		checkSlow(DstNumIndexes % 3 == 0);
		// Copy the vertices over
		TMap<int32, FVertexID> VertexIDMap;
		VertexIDMap.Reserve(DstNumPositions);
		{
			const auto& AOSVertexes = AOSMesh.Vertexes;
			for (uint32 i = 0, I = DstNumPositions; i < I; ++i)
			{
				const FVector3f& Position = AOSVertexes[i].GetPos();
				const FVertexID NewVertexID = OutRawMesh.CreateVertex();
				VertexPositions[NewVertexID] = Position;
				VertexIDMap.Add(i, NewVertexID);
			}

			checkSlow(VertexPositions.GetNumElements() == DstNumPositions);
		}

		const uint32* AOSIndexes = AOSMesh.Indexes;

		// Connectivity: 
		auto CreateTriangle = [&OutRawMesh, PolygonGroupID, &VertexInstanceNormals, &VertexInstanceTangents, &VertexInstanceBinormalSigns, &VertexInstanceColors, &VertexInstanceUVs, &EdgeHardnesses](const FVertexID TriangleIndex[3], const FVector3f Normals[3])
			{
				TArray<FVertexInstanceID> VertexInstanceIDs;
				VertexInstanceIDs.SetNum(3);
				for (int32 Corner = 0; Corner < 3; ++Corner)
				{
					VertexInstanceIDs[Corner] = OutRawMesh.CreateVertexInstance(TriangleIndex[Corner]);
					VertexInstanceTangents[VertexInstanceIDs[Corner]] = FVector3f(1, 0, 0);
					VertexInstanceNormals[VertexInstanceIDs[Corner]] = Normals[Corner];
					VertexInstanceBinormalSigns[VertexInstanceIDs[Corner]] = GetBasisDeterminantSign((FVector)VertexInstanceTangents[VertexInstanceIDs[Corner]].GetSafeNormal(),
						(FVector)(VertexInstanceNormals[VertexInstanceIDs[Corner]] ^ VertexInstanceTangents[VertexInstanceIDs[Corner]]).GetSafeNormal(),
						(FVector)VertexInstanceNormals[VertexInstanceIDs[Corner]].GetSafeNormal());
					VertexInstanceColors[VertexInstanceIDs[Corner]] = FVector4f(1.0f);
					VertexInstanceUVs.Set(VertexInstanceIDs[Corner], 0, FVector2f(0.0f, 0.0f));
				}

				// Insert a polygon into the mesh
				OutRawMesh.CreatePolygon(PolygonGroupID, VertexInstanceIDs);
			};

		{
			uint32 IndexStop = DstNumIndexes / 3;
			for (uint32 t = 0, T = IndexStop; t < T; ++t)
			{
				FVertexID VertexIndexes[3];
				FVector3f Normals[3];
				for (int32 Corner = 0; Corner < 3; ++Corner)
				{
					VertexIndexes[Corner] = VertexIDMap[AOSIndexes[(t * 3) + Corner]];
					const auto& AOSVertex = AOSMesh.Vertexes[AOSIndexes[(t * 3) + Corner]];
					Normals[Corner] = AOSVertex.Normal;
				}
				CreateTriangle(VertexIndexes, Normals);
			}
		}
	}

	void ConvertMesh(const FAOSMesh& InMesh, FMeshDescription& OutMesh)
	{
		AOSMeshToRawMesh(InMesh, OutMesh);
	}

	/* ---------------------------------------------------------------------------------------------------- */

	FMatrix CreateMatrixFromTransformAndVoxelSize(
		const FVector InTranslation,
		const FVector InRotation,
		const FVector InScale,
		const float InVoxelSize)
	{
		const FQuat Rotation = FQuat(FRotator(InRotation.Y, InRotation.Z, InRotation.X));
		FTransform MeshXForm(Rotation, InTranslation, InScale);

		FMatrix LocalToVoxel = FMatrix::Identity;
		LocalToVoxel.M[0][0] = InVoxelSize;
		LocalToVoxel.M[1][1] = InVoxelSize;
		LocalToVoxel.M[2][2] = InVoxelSize;

		FMatrix TransformMatrix = MeshXForm.ToMatrixWithScale().Inverse();
		TransformMatrix = LocalToVoxel * TransformMatrix;

		return TransformMatrix;
	}

	openvdb::Mat4R ConvertMatrixToVDBMatrix(FMatrix& InMatrix)
	{
		double* data = &InMatrix.M[0][0];
		openvdb::Mat4R VDBMatDouble(data);

		// NB: rounding errors in the inverse may have resulted in error in this col.
		// openvdb explicitly checks this matrix row to insure the transform is affine and will throw 
		VDBMatDouble.setCol(3, openvdb::Vec4R(0, 0, 0, 1));

		return VDBMatDouble;
	}

	/* ---------------------------------------------------------------------------------------------------- */

}

