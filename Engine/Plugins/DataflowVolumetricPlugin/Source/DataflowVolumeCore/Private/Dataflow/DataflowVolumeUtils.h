// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/OpenVDB.h"
#include "Math/Vector.h"
#include "MeshDescription.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#include <tbb/blocked_range.h>
#include <tbb/blocked_range2d.h>
#include <tbb/concurrent_vector.h>
#include <tbb/parallel_for.h>
#include <tbb/parallel_reduce.h>
#include <tbb/task_group.h>
#include <tbb/task_scheduler_observer.h>
PRAGMA_ENABLE_DEPRECATION_WARNINGS
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

struct FMeshDescription;

namespace UE::DataflowVolumeUtils
{
	class FPlacedMesh
	{
	public:
		FPlacedMesh()
			: Mesh(nullptr)
		{}

		explicit FPlacedMesh(const FMeshDescription* MeshIn, const FTransform& TransformIn = FTransform::Identity)
			: Mesh(MeshIn), Transform(TransformIn)
		{}

		FPlacedMesh(const FPlacedMesh& other)
			: Mesh(other.Mesh)
			, Transform(other.Transform)
		{}

		FPlacedMesh& operator=(const FPlacedMesh& other)
		{
			Mesh = other.Mesh;
			Transform = other.Transform;
			return *this;
		}

		const FMeshDescription* Mesh;
		FTransform       Transform;
	};

	// the interrupter interface. 
	struct FInterrupter
	{
		FInterrupter() = default;
		virtual ~FInterrupter() = default;
		// templated openvdb code requires these functions.
		virtual void start(const char* name = nullptr) { (void)name; }
		virtual void end() {};
		virtual bool wasInterrupted(int percent = -1) { (void)percent; return false; }
	};

	/**
	*  Mesh adapter class that implements the needed API for the methods in openvdb that convert
	*  polygonal mesh to a signed distance field.
	*/
	class FMeshDescriptionAdapter
	{
	public:
		/**
		* Constructor
		*/
		FMeshDescriptionAdapter(const FMeshDescription& InRawMesh, const openvdb::math::Transform& InTransform);
		FMeshDescriptionAdapter(const FMeshDescriptionAdapter& other);

		// Total number of polygons
		size_t polygonCount() const;

		// Total number of points (vertex locations)
		size_t pointCount() const;

		// Vertex count for polygon n: currently FMeshDescription is just triangles.
		size_t vertexCount(size_t n) const { return 3; }

		// Return position pos in local grid index space for polygon n and vertex v
		void getIndexSpacePoint(size_t FaceNumber, size_t CornerNumber, openvdb::Vec3d& pos) const;

		/**
		* The transform used to map between the physical space of the mesh and the voxel space.
		*/
		const openvdb::math::Transform& GetTransform() const
		{
			return Transform;
		}

	private:

		// Pointer to the raw mesh we are wrapping
		const FMeshDescription* RawMesh;



		//////////////////////////////////////////////////////////////////////////
		//Cache data
		void InitializeCacheData();
		uint32 TriangleCount;
		TVertexAttributesConstRef<FVector3f> VertexPositions;
		// Local version of the index array.  The FMeshDescription doesn't really have one.
		TArray<FVertexInstanceID> IndexBuffer;



		// Transform used to convert the mesh space into the index space used by voxelization
		const openvdb::math::Transform Transform;
	};



	/**
	* Mesh type that holds minimal required data for openvdb iso-surface extraction
	* interface. Implements the openvdb MeshDataAdapter interface.
	*
	* NB: std::vector required by the openvdb interface.
	*/
	struct FMixedPolyMesh
	{
		std::vector<openvdb::Vec3s> Points;
		std::vector<openvdb::Vec4I> Quads;
		std::vector<openvdb::Vec3I> Triangles;
		openvdb::math::Transform	Transform;

		// ~MeshDataAdapter Interface Begin
		// https://www.openvdb.org/documentation/doxygen/interfaceMeshDataAdapter.html

		// Total number of polygons
		size_t polygonCount() const
		{
			return Quads.size() + Triangles.size();
		}

		// Total number of points (vertex locations)
		size_t pointCount() const
		{
			return Points.size();
		}

		// Vertex count for polygon n
		size_t vertexCount(size_t n) const
		{
			return PolygonIsQuad(n) ? 4 : 3;
		}

		// Return position pos in local grid index space for polygon n and vertex v
		void getIndexSpacePoint(size_t n, size_t v, openvdb::Vec3d& pos) const
		{
			if (PolygonIsQuad(n))
			{
				pos = Points[Quads[GetQuadIndex(n)][v]];
			}
			else
			{
				pos = Points[Triangles[GetTriangleIndex(n)][v]];
			}

			pos = Transform.worldToIndex(pos);
		}
		// ~MeshDataAdapter Interface End

	private:
		bool PolygonIsQuad(size_t n) const
		{
			return n < Quads.size();
		}

		bool PolygonIsTriangle(size_t n) const
		{
			return n >= Triangles.size();
		}

		size_t GetQuadIndex(size_t n) const
		{
			return n;
		}

		size_t GetTriangleIndex(size_t n) const
		{
			return n - Quads.size();
		}
	};



	/**
	* Simplifier vertex type that stores position and normal.
	*
	* Implements the interface needed by templated simplifier code.
	*/
	class FPositionNormalVertex
	{
		typedef FPositionNormalVertex  VertType;
	public:
		FPositionNormalVertex() {}

		enum { NumAttributesValue = 3 };

		uint32          MaterialIndex = 0;
		FVector3f			Position;
		// ---- Attributes --------
		// currently just the components of the normal.
		FVector3f			Normal;


		// Attributes are all assumed to be floats.  
		static uint32 NumAttributes()
		{

			return (sizeof(VertType) - sizeof(uint32) - sizeof(FVector3f)) / sizeof(float);
		}

		// Methods needed by the simplifier.
		uint32			GetMaterialIndex() const { return MaterialIndex; }
		FVector3f& GetPos() { return Position; }
		const FVector3f& GetPos() const { return Position; }
		float* GetAttributes() { return (float*)&Normal; }
		const float* GetAttributes() const { return (const float*)&Normal; }


		void Correct() { Normal.Normalize(); }

		// Note this uses exact float compares..
		bool operator==(const VertType& OtherVert) const
		{
			bool Result = true;
			if (MaterialIndex != OtherVert.MaterialIndex ||
				Position != OtherVert.Position ||
				Normal != OtherVert.Normal)
			{
				Result = false;
			}

			return Result;
		}
		bool operator!=(const VertType& OtherVert)
		{
			return !(*this == OtherVert);
		}

		// NB: Addition isn't commutative since the MaterialIndex..
		// maybe Result.MaterialIndex = min(this->MaterialIndex, Other.MaterialIndex) 
		// would be better.
		VertType operator+(const VertType& OtherVert) const
		{
			VertType Result(*this);

			Result.Position += OtherVert.Position;
			Result.Normal += OtherVert.Normal;

			return Result;
		}

		// NB: A-B != -(B-A) because of the material index.
		VertType operator-(const VertType& OtherVert) const
		{
			VertType Result(*this);

			Result.Position -= OtherVert.Position;
			Result.Normal -= OtherVert.Normal;

			return Result;
		}

		// NB: The normal isn't unit length after this..
		VertType operator*(const float Scalar) const
		{
			VertType Result(*this);
			Result.Position *= Scalar;
			Result.Normal *= Scalar;

			return Result;
		}


		VertType operator/(const float Scalar) const
		{
			float ia = 1.0f / Scalar;
			return (*this) * ia;
		}
	};



	/**
	* A Triangle Mesh Array Of Structs (AOS) for the vertex data.  This should work with the simplification code.
	*/
	template <typename SimplifierVertexType>
	class TAOSMesh
	{
	public:

		typedef SimplifierVertexType  VertexType;

		TAOSMesh();
		TAOSMesh(int32 VertCount, int32 FaceCount);

		~TAOSMesh();

		// reset the mesh to size zero, deleting content.
		void Empty();

		// resize the mesh, deleting the current content.
		void Resize(int32 VertCount, int32 FaceCount);

		// set the real vertex & index count after duplicate removal
		void SetVertexAndIndexCount(uint32 VertCount, uint32 IndexCount);

		// Swap content with an existing mesh of the same type.
		void Swap(TAOSMesh& other);


		// Get the indices that correspond to this face.
		openvdb::Vec3I GetFace(int32 FaceNumber) const;

		// Get an array of positions only.
		void GetPosArray(std::vector<FVector3f>& PosArray) const;


		// This method is for testing only.  Not designed for perf.
		int32 ComputeDegenerateTriCount() const;


		bool IsEmpty() const { return (NumVertexes == 0 || NumIndexes == 0); }
		uint32 GetNumVertexes() const { return NumVertexes; }
		uint32 GetNumIndexes()  const { return NumIndexes; }

		// unfettered public access.
		VertexType* Vertexes;
		uint32* Indexes;

	private:

		uint32 NumVertexes;
		uint32 NumIndexes;

		TAOSMesh(const TAOSMesh& other);
	};



	/**
	* Array of structs mesh composed of vertices that are compatible with the simplifier.
	*/
	typedef TAOSMesh<FPositionNormalVertex>  FAOSMesh;

	/* Threaded wrappers */
	typedef tbb::blocked_range<int32>    FIntRange;
	typedef tbb::blocked_range<uint32>   FUIntRange;
	typedef tbb::blocked_range2d<int32>  FIntRange2d;

	template <typename RangeType, typename FunctorType>
	void Parallel_For(const RangeType& Range, const FunctorType& Functor, const bool bParallel = true)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::Parallel_For)
			if (bParallel) // run in parallel
			{
				// Functor can be passed by reference since we wait until completion
				tbb::parallel_for(Range,
					[&Functor](const RangeType& Range)
					{
						// #TODO Investigate why Insights stops working when used
						//TRACE_CPUPROFILER_EVENT_SCOPE(ProxyLOD::Parallel_For)
						Functor(Range);
					});
			}
			else // single threaded
			{
				Functor(Range);
			}
	}

	// --- Implementation of templated code. ---


	template <typename SimplifierVertexType>
	TAOSMesh<SimplifierVertexType>::TAOSMesh() :
		Vertexes(NULL),
		Indexes(NULL),
		NumVertexes(0),
		NumIndexes(0)
	{}

	template <typename SimplifierVertexType>
	TAOSMesh<SimplifierVertexType>::TAOSMesh(int32 VertCount, int32 FaceCount) :
		Vertexes(NULL),
		Indexes(NULL),
		NumVertexes(0),
		NumIndexes(0)
	{
		this->Resize(VertCount, FaceCount);
	}

	template <typename SimplifierVertexType>
	TAOSMesh<SimplifierVertexType>::~TAOSMesh()
	{
		this->Empty();
	}

	template <typename SimplifierVertexType>
	void TAOSMesh<SimplifierVertexType>::Empty()
	{
		if (Vertexes) delete[] Vertexes;
		if (Indexes)  delete[] Indexes;

		Vertexes = NULL;
		Indexes = NULL;
		NumVertexes = 0;
		NumIndexes = 0;
	}

	template <typename SimplifierVertexType>
	void TAOSMesh<SimplifierVertexType>::Resize(int32 VertCount, int32 FaceCount)
	{
		check(VertCount > -1); check(FaceCount > -1);

		this->Empty();

		if (FaceCount > 0 && VertCount > 0)
		{
			NumVertexes = VertCount;
			NumIndexes = FaceCount * 3;

			Vertexes = new VertexType[VertCount];
			Indexes = new uint32[NumIndexes];
		}

	}

	template <typename SimplifierVertexType>
	void TAOSMesh<SimplifierVertexType>::SetVertexAndIndexCount(uint32 VertCount, uint32 IndexCount)
	{
		check(VertCount <= NumVertexes);
		check(IndexCount <= NumIndexes);

		NumVertexes = VertCount;
		NumIndexes = IndexCount;
	}

	template <typename SimplifierVertexType>
	void TAOSMesh<SimplifierVertexType>::Swap(TAOSMesh& other)
	{
		// Swap sizes
		{
			uint32 Tmp = other.NumVertexes;
			other.NumVertexes = NumVertexes;
			NumVertexes = Tmp;

			Tmp = other.NumIndexes;
			other.NumIndexes = NumIndexes;
			NumIndexes = Tmp;
		}

		// Swap Vertex Pointer
		{
			VertexType* TmpVp = other.Vertexes;
			other.Vertexes = Vertexes;
			Vertexes = TmpVp;
		}

		// Sway index pointer
		{
			uint32* TmpIdp = other.Indexes;
			other.Indexes = Indexes;
			Indexes = TmpIdp;
		}

	}

	template <typename SimplifierVertexType>
	openvdb::Vec3I TAOSMesh<SimplifierVertexType>::GetFace(int32 FaceNumber) const
	{
		check(FaceNumber > -1); check(uint32(FaceNumber * 3) < NumIndexes);

		uint32 offset = FaceNumber * 3;
		return openvdb::Vec3I(Indexes[offset], Indexes[offset + 1], Indexes[offset + 2]);
	}

	template <typename SimplifierVertexType>
	void TAOSMesh<SimplifierVertexType>::GetPosArray(std::vector<FVector3f>& PosArray) const
	{
		// resize the target array
		PosArray.resize(NumVertexes);

		// copy the data over.
		Parallel_For(FUIntRange(0, NumVertexes), [this, &PosArray](const FUIntRange& Range)
			{
				FVector3f* Pos = PosArray.data();
				FPositionNormalVertex* VertexArray = this->Vertexes;


				for (uint32 r = Range.begin(), R = Range.end(); r < R; ++r)
				{
					Pos[r] = VertexArray[r].GetPos();
				}
			});
	}

	template <typename SimplifierVertexType>
	int32 TAOSMesh<SimplifierVertexType>::ComputeDegenerateTriCount() const
	{
		int32 DegenerateTriCount = 0;
		uint32 NumTris = GetNumIndexes() / 3;
		for (uint32 i = 0; i < NumTris; ++i)
		{
			uint32 Offset = i * 3;
			uint32 Idx[3] = { Indexes[Offset], Indexes[Offset + 1], Indexes[Offset + 2] };
			FVector Tri[3] = { Vertexes[Idx[0]].GetPos(), Vertexes[Idx[1]].GetPos(), Vertexes[Idx[2]].GetPos() };

			const FVector NormalDir = (Tri[2] - Tri[0]) ^ (Tri[1] - Tri[0]);

			if (NormalDir.SizeSquared() == 0.0f)
			{
				DegenerateTriCount++;
			}
		}

		return DegenerateTriCount;
	}



	/**
	* Convert a mixed triangle and quad mesh to a triangle mesh type by splitting quads.
	* No new vertices are created.
	*
	* NB: This does not attempt to add any additional attributes to the result.  Will
	*     need to separately compute vertex normals if desired.
	*
	* @param InMesh   Mesh with a mixture of quads and triangles.
	* @param OutMesh  Triangle Mesh.
	* @param bClockWise how to order the verts in a triangle
	*/
	template <typename T>
	void MixedPolyMeshToAOSMesh(const FMixedPolyMesh& InMesh, TAOSMesh<T>& OutMesh, const bool bClockWise = true);

	// Convert MixedPolyMesh to AOS Mesh.  This requires splitting quads to produce triangles.
	template <typename T>
	void MixedPolyMeshToAOSMesh(const FMixedPolyMesh& MixedPolyMesh, TAOSMesh<T>& DstAOSMesh, bool bClockWise)
	{

		// Splitting a quad doesn't introduce any new verts.
		const uint32 DstNumVerts = MixedPolyMesh.Points.size();

		const uint32 NumQuads = MixedPolyMesh.Quads.size();

		// Each quad becomes 2 triangles.
		const uint32 DstNumTris = 2 * NumQuads + MixedPolyMesh.Triangles.size();

		// Each Triangle has 3 corners
		const uint32 DstNumIndexes = 3 * DstNumTris;

		// Empty and Allocate space

		DstAOSMesh.Resize(DstNumVerts, DstNumTris);

		// Copy the vertices position over and give it a dummy material index.
		{
			// Allocate the space for the verts in the DstAOSMesh

			Parallel_For(FUIntRange(0, DstNumVerts),
				[&MixedPolyMesh, &DstAOSMesh](const FUIntRange& Range)
				{
					for (uint32 i = Range.begin(), I = Range.end(); i < I; ++i)
					{
						const openvdb::Vec3s& Point = MixedPolyMesh.Points[i];
						DstAOSMesh.Vertexes[i].Position = FVector3f(Point[0], Point[1], Point[2]);
						DstAOSMesh.Vertexes[i].MaterialIndex = 0;
					}
				});
		}

		// Split VDB Quads
		{


			// NB: The Quads are ordered in clockwise fashion.

			Parallel_For(FUIntRange(0, NumQuads),
				[&MixedPolyMesh, &DstAOSMesh, bClockWise](const FUIntRange& Range)
				{
					uint32* Indices = DstAOSMesh.Indexes;
					for (uint32 q = Range.begin(), Q = Range.end(); q < Q; ++q)
					{
						const uint32 Offset = q * 6;
						const openvdb::Vec4I& Quad = MixedPolyMesh.Quads[q];
						// add as two triangles
						if (bClockWise)
						{
							// first triangle
							Indices[Offset] = Quad[0];
							Indices[Offset + 1] = Quad[1];
							Indices[Offset + 2] = Quad[2];
							// second triangle
							Indices[Offset + 3] = Quad[2];
							Indices[Offset + 4] = Quad[3];
							Indices[Offset + 5] = Quad[0];
						}
						else
						{
							// first triangle
							Indices[Offset] = Quad[0];
							Indices[Offset + 1] = Quad[3];
							Indices[Offset + 2] = Quad[2];

							// second triangle
							Indices[Offset + 3] = Quad[2];
							Indices[Offset + 4] = Quad[1];
							Indices[Offset + 5] = Quad[0];
						}
					}

				});

			// add the MixedPolyMesh triangles.
			Parallel_For(FUIntRange(0, MixedPolyMesh.Triangles.size()),
				[&MixedPolyMesh, &DstAOSMesh, NumQuads, bClockWise](const FUIntRange& Range)
				{
					uint32* Indices = DstAOSMesh.Indexes;
					for (uint32 t = Range.begin(), EndT = Range.end(); t < EndT; ++t)
					{
						const uint32 Offset = NumQuads * 6 + t * 3;
						const openvdb::Vec3I& Tri = MixedPolyMesh.Triangles[t];
						// add the triangle
						if (bClockWise)
						{
							Indices[Offset] = Tri[0];
							Indices[Offset + 1] = Tri[1];
							Indices[Offset + 2] = Tri[2];
						}
						else
						{
							Indices[Offset] = Tri[2];
							Indices[Offset + 1] = Tri[1];
							Indices[Offset + 2] = Tri[0];
						}
					}
				});
		}
	}



	/**
	* Common interface to convert between various triangle-based mesh types.
	*
	* The conversions will maintain geometry and connectivity, but different mesh types support
	* different attribute types and possibly different frequenceies even when the same attribute types
	* exist on both mesh types ( e.g. FMeshDescription has per-wedge data while FVertexDataMesh has per-vertex attributes).
	*
	* @param InMesh   Source Mesh to convert.
	* @param OutMesh  The result of the mesh conversion.  Any data already stored in OutMesh will be lost.
	*
	*/
	void ConvertMesh(const FAOSMesh& InMesh, FMeshDescription& OutMesh);




	/**
	* Convert the simplifier-friendly array-of-structs mesh to a struct of arrays FMeshDescription.
	*
	* NB: This copies the AOSVertex normal to the wedge FMeshDescription::Tangentz.
	*     But additional FMeshDescription attributes, including tangent/bitangent are given default values.
	*
	* @param InMesh   Source Mesh to convert.
	* @param OutMesh  The result of the mesh conversion.  Any data already stored in OutMesh will be lost.
	*
	*/
	void AOSMeshToRawMesh(const FAOSMesh& InMesh, FMeshDescription& OutMesh);

	/**
	* Create a world-to-index-space transform matrix from transform and voxelSize
	*/
	FMatrix CreateMatrixFromTransformAndVoxelSize(
		const FVector InTranslation, 
		const FVector InRotation,
		const FVector InScale,
		const float InVoxelSize);

	/**
	* Convert a FMatrix to openvdb::Mat4R
	*/
	openvdb::Mat4R ConvertMatrixToVDBMatrix(FMatrix& InMatrix);

	template <typename GridType>
	void GetActiveVoxels(const typename GridType::Ptr& InGridPtr, TArray<FBox>& OutActiveVoxels)
	{
		using IterT = typename GridType::ValueOnCIter;

		openvdb::math::Transform Transform = InGridPtr->transform();

		if (!InGridPtr->tree().empty())
		{
			for (IterT iter = InGridPtr->cbeginValueOn(); iter.test(); ++iter)
			{
				if (iter.isVoxelValue())
				{
					openvdb::CoordBBox BBox;
					iter.getBoundingBox(BBox);

					const openvdb::Vec3d BBoxMin(BBox.min().x() - 0.5, BBox.min().y() - 0.5, BBox.min().z() - 0.5);
					const openvdb::Vec3d BBoxMax(BBox.max().x() + 0.5, BBox.max().y() + 0.5, BBox.max().z() + 0.5);

					const openvdb::Vec3d BBoxMinWS = Transform.indexToWorld(BBoxMin);
					const openvdb::Vec3d BBoxMaxWS = Transform.indexToWorld(BBoxMax);

					FVector Min(BBoxMinWS.x(), BBoxMinWS.y(), BBoxMinWS.z());
					FVector Max(BBoxMaxWS.x(), BBoxMaxWS.y(), BBoxMaxWS.z());

					OutActiveVoxels.Add(FBox(Min, Max));
				}
			}
		}
	}

	/**
	* Get active voxels of a volume of any type
	*/
	template <typename GridType>
	void GetActiveVoxels(const typename GridType::Ptr& InGridPtr, const float InIsovalue, TArray<FBox>& OutActiveVoxels, bool bInteriorMaskOnly = false)
	{
		if (!InGridPtr->tree().empty())
		{
			if (bInteriorMaskOnly)
			{
				// Compute a mask of the voxels enclosed by the isosurface
				auto interiorMaskPtr = [&InGridPtr, &InIsovalue]
					{
						if constexpr (std::is_same_v<GridType, openvdb::FloatGrid>)
						{
							return openvdb::tools::sdfInteriorMask(*InGridPtr, InIsovalue);
						}
						else
						{
							return openvdb::tools::interiorMask(*InGridPtr, InIsovalue);
						}
					} ();

				GetActiveVoxels<openvdb::BoolGrid>(interiorMaskPtr, OutActiveVoxels);
			}
			else
			{
				GetActiveVoxels<GridType>(InGridPtr, OutActiveVoxels);
			}
		}
	}
}
