// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START

UE_PUSH_MACRO("check")
#undef check

#ifdef _MSC_VER
	#pragma warning (push)
	#pragma warning (disable : 4146) // unary minus operator applied to unsigned type, result still unsigned
	#pragma warning (disable : 4706) // assignment within conditional expression
#endif

#ifndef M_PI
    #define M_PI    3.14159265358979323846
	#define LOCAL_M_PI 1
#endif

#ifndef M_PI_2
    #define M_PI_2  1.57079632679489661923 // pi/2
	#define LOCAL_M_PI_2 1
#endif

#define BOOST_ALLOW_DEPRECATED_HEADERS

#include <openvdb/openvdb.h>
#include <openvdb/tools/Composite.h> 		// for csgUnion
#include <openvdb/tools/Interpolation.h> 	// for grid sampler
#include <openvdb/tools/MeshToVolume.h> 	// for MeshToVolume
#include <openvdb/tools/VolumeToMesh.h> 	// for VolumeToMesh
#include <openvdb/tools/LevelSetSphere.h>   // Generate a narrow-band level set of sphere
#include <openvdb/tools/LevelSetPlatonic.h> // Generate a narrow-band level sets of the five platonic solids
#include <openvdb/points/PointCount.h>
#include <openvdb/tools/VolumeToSpheres.h>
#include <openvdb/tools/Morphology.h>
#include <openvdb/math/Math.h> // for isFinite()
#include <openvdb/tools/ChangeBackground.h>
#include <openvdb/tools/GridTransformer.h> // for resampleToMatch()
#include <openvdb/tools/LevelSetRebuild.h> // for levelSetRebuild()
#include <openvdb/tools/Prune.h>
#include <openvdb/tools/SignedFloodFill.h>
#include <openvdb/util/NullInterrupter.h>
#include <openvdb/Types.h>
#include <openvdb/points/PointScatter.h>
#include <openvdb/math/Math.h>
#include <openvdb/tools/LevelSetUtil.h> 
#include <openvdb/tools/LevelSetFracture.h>
#include <openvdb/tools/ParticlesToLevelSet.h>
#include <openvdb/tools/GridOperators.h>
#include <openvdb/tools/Mask.h>
#if LOCAL_M_PI
	#undef M_PI
#endif

#if LOCAL_M_PI_2
	#undef M_PI_2
#endif

#ifdef _MSC_VER
	#pragma warning (pop)
#endif

UE_POP_MACRO("check")

THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif