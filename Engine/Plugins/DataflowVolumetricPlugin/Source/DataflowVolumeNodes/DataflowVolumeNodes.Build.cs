// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataflowVolumeNodes : ModuleRules
	{
        public DataflowVolumeNodes (ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] 
				{
					"DynamicMesh",
					"MeshConversion",
					"Core",
					"CoreUObject",
					"ChaosCore",
					"Chaos",
					"DataflowCore",
					"DataflowEngine",
					"DataflowEnginePlugin",
					"GeometryCollectionEngine",
					"GeometryCore",
					"Voronoi",
					"GeometryFramework",
					"MeshDescription",
					"StaticMeshDescription",
					"PlanarCut",
					"Engine",
					"DataflowEditor",
					"DataflowVolumeCore"
				}
			);
		}
	}
}
