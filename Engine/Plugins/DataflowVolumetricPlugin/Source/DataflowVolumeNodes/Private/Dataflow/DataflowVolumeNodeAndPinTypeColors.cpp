// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVolumeNodeAndPinTypeColors.h"
#include "Dataflow/DataflowNode.h"
#include "Math/Color.h" 
#include "Dataflow/DataflowNodeColorsRegistry.h"

namespace UE::Dataflow
{
	void RegisterVolumeColors()
	{
		static const FLinearColor CDefaultNodeBodyTintColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);
		static const float CDefaultWireThickness = 1.5f;

		// Node colors
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Volume", FLinearColor(0.273f, 0.172f, 1.f), CDefaultNodeBodyTintColor);

		// PinType colors
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowFloatVolume", FLinearColor(.263f, 0.412f, 0.9f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowIntVolume", FLinearColor(.363f, 0.512f, 0.9f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowVectorVolume", FLinearColor(.463f, 0.612f, 0.9f, 1.0f), CDefaultWireThickness);
	}
}
