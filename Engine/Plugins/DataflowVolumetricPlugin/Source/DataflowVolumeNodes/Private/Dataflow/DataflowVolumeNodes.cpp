// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVolumeNodes.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowCore.h"
#include "StaticMeshAttributes.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Dataflow/DataflowOverlay.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#if WITH_EDITOR
#include "Dataflow/DataflowRenderingViewMode.h"
#endif
#include "GeometryCollection/Facades/PointsCollectionFacade.h"
#include "Engine/VolumeTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowVolumeNodes)

namespace UE::Dataflow
{
	void RegisterVolumeNodes()
	{
		// Generators
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeSDFSphereDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeSDFCubeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeSDFPlatonicSolidDataflowNode);
		// Utilities
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVolumeInfoDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVolumeAnalysisDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVolumeCombineDataflowNode);
		// Convert
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionToVolumeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVolumeToSpheresDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVolumeToSpheresDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertVolumeDataflowNode);

		// Scatter
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUniformVolumeScatterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDenseUniformVolumeScatterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FNonUniformVolumeScatterDataflowNode);

		// Sample
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVolumeSampleToPointsDataflowNode);

		// Visualize
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVolumeSliceDataflowNode);

		// Terminal
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVolumeTextureTerminalNode);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVolumeToSpheresWithRelaxationDataflowNode);
	}
}

/* --------------------------------------------------------------------------------------------- */

FMakeSDFSphereDataflowNode::FMakeSDFSphereDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&VoxelSize);
	RegisterInputConnection(&Radius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Center).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&FloatVolume);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.bTranslucent = false;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 1.5f;
}

void FMakeSDFSphereDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FloatVolume))
	{
		const float InVoxelSize = GetValue(Context, &VoxelSize);
		const float InRadius = GetValue(Context, &Radius);
		const FVector InCenter = GetValue(Context, &Center);

		FDataflowFloatVolume OutFloatVolume = FDataflowFloatVolume::CreateSphereSDF(InVoxelSize, InRadius, InCenter);

		SetValue(Context, OutFloatVolume, &FloatVolume);
	}
}

/* --------------------------------------------------------------------------------------------- */

FMakeSDFCubeDataflowNode::FMakeSDFCubeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Scale).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Center).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&VoxelSize).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&FloatVolume);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.bTranslucent = false;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 1.5f;
}

void FMakeSDFCubeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FloatVolume))
	{
		const float InScale = GetValue(Context, &Scale);
		const FVector InCenter = GetValue(Context, &Center);
		const float InVoxelSize = GetValue(Context, &VoxelSize);

		FDataflowFloatVolume OutFloatVolume = FDataflowFloatVolume::CreateCubeSDF(
			InScale,
			InCenter,
			InVoxelSize);

		SetValue(Context, OutFloatVolume, &FloatVolume);
	}
}

/* --------------------------------------------------------------------------------------------- */

FMakeSDFPlatonicSolidDataflowNode::FMakeSDFPlatonicSolidDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Scale).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Center).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&VoxelSize).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&FloatVolume);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.bTranslucent = false;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 1.5f;
}

void FMakeSDFPlatonicSolidDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FloatVolume))
	{
		int32 FaceCount = 0;
		switch (PlatonicSolidType)
		{
			case EDataflowVolumePlatonicSolidType::Tetrahedron:
				FaceCount = 4;
				break;
			case EDataflowVolumePlatonicSolidType::Cube:
				FaceCount = 6;
				break;
			case EDataflowVolumePlatonicSolidType::Octahedron:
				FaceCount = 8;
				break;
			case EDataflowVolumePlatonicSolidType::Dodecahedron:
				FaceCount = 12;
				break;
			case EDataflowVolumePlatonicSolidType::Icosahedron:
				FaceCount = 20;
				break;
			default:
				break;
		}

		const float InScale = GetValue(Context, &Scale);
		const FVector InCenter = GetValue(Context, &Center);
		const float InVoxelSize = GetValue(Context, &VoxelSize);

		FDataflowFloatVolume OutFloatVolume = FDataflowFloatVolume::CreatePlatonicSolidSDF(
			FaceCount,
			InScale,
			InCenter,
			InVoxelSize);

		SetValue(Context, OutFloatVolume, &FloatVolume);
	}
}

/* --------------------------------------------------------------------------------------------- */

FVolumeInfoDataflowNode::FVolumeInfoDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&FloatVolume);
	RegisterOutputConnection(&String);
}

void FVolumeInfoDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&String))
	{
		const FDataflowFloatVolume& InFloatVolume = GetValue(Context, &FloatVolume);

		FString OutString = InFloatVolume.VolumeInfo();

		SetValue(Context, MoveTemp(OutString), &String);
	}
}

/* --------------------------------------------------------------------------------------------- */

FCollectionToVolumeDataflowNode::FCollectionToVolumeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&ReferenceVolume).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&VoxelSize).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&IsoValue).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&FloatVolume);
	RegisterOutputConnection(&FaceIndexVolume);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Shaded;
	DebugDrawRenderSettings.bTranslucent = false;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 1.5f;
}

void FCollectionToVolumeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FloatVolume) || Out->IsA(&FaceIndexVolume))
	{
		if (IsConnected(&Collection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const float InVoxelSize = GetValue(Context, &VoxelSize);

			// Anytype returns storage type
			FDataflowVolume InReferenceVolume = GetValue(Context, &ReferenceVolume);

			if (InCollection.NumElements(FGeometryCollection::TransformGroup) > 0)
			{
				if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
				{
					const TManagedArray<FTransform3f>& BoneTransforms = InCollection.GetAttribute<FTransform3f>("Transform", FGeometryCollection::TransformGroup);

					TArray<int32> TransformIndices;
					TransformIndices.AddUninitialized(BoneTransforms.Num());

					int32 Idx = 0;
					for (int32& TransformIdx : TransformIndices)
					{
						TransformIdx = Idx++;
					}

					FMeshDescription MeshDescription;
					FStaticMeshAttributes Attributes(MeshDescription);
					Attributes.Register();

					FTransform TransformOut;

					ConvertToMeshDescription(MeshDescription, TransformOut, /*bCenterPivot*/true, *GeomCollection, BoneTransforms, TransformIndices);

					FDataflowIntVolume OutFaceIndexVolume;

					FDataflowFloatVolume OutVolume = FDataflowFloatVolume::CreateVolumeFromMeshDescription(
						MeshDescription,
						TransformOut,
						InVoxelSize,
						InReferenceVolume,
						OutputType,
						GridName,
						bUseWorldSpaceUnits,
						ExteriorBand,
						InteriorBand,
						ExteriorBandVoxels,
						InteriorBandVoxels,
						bFillInterior,
						bPreserveHoles,
						IsoValue,
						OutFaceIndexVolume);

					SetValue(Context, OutVolume, &FloatVolume);
					SetValue(Context, OutFaceIndexVolume, &FaceIndexVolume);

					return;
				}
			}
		}

		SetValue(Context, FDataflowFloatVolume(), &FloatVolume);
		SetValue(Context, FDataflowIntVolume(), &FaceIndexVolume);
	}
}

/* --------------------------------------------------------------------------------------------- */

FVolumeToSpheresDataflowNode::FVolumeToSpheresDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&FloatVolume);
	RegisterInputConnection(&MinSphereCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxSphereCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&IsoValue).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&InstanceCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&Spheres);
}

void FVolumeToSpheresDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) || Out->IsA(&Spheres))
	{
		if (IsConnected(&FloatVolume))
		{
			const FDataflowFloatVolume& InFloatVolume = GetValue(Context, &FloatVolume);
			const int32 InMinSphereCount = GetValue(Context, &MinSphereCount);
			const int32 InMaxSphereCount = GetValue(Context, &MaxSphereCount);
			const float InMinRadius = GetValue(Context, &MinRadius);
			const float InMaxRadius = GetValue(Context, &MaxRadius);
			const float InIsoValue = GetValue(Context, &IsoValue);
			const int32 InInstanceCount = GetValue(Context, &InstanceCount);

			TArray<FVector> OutSphereCenters;
			TArray<float> OutSphereRadii;

			// Compute sphere packing
			InFloatVolume.VolumeToSpheres(
				InMinSphereCount,
				InMaxSphereCount,
				bOverlapping,
				InMinRadius,
				InMaxRadius,
				InIsoValue,
				InInstanceCount,
				OutSphereCenters,
				OutSphereRadii);

			// Add spheres to OutCollection
			FManagedArrayCollection OutCollection;

			const int32 NumSpheresFromSDF = OutSphereCenters.Num();

			if (NumSpheresFromSDF > 0)
			{
				// Add new element to groups
				static const FName SpheresGroup = "Spheres";

				const int32 NumTransforms = OutCollection.NumElements(FGeometryCollection::TransformGroup);
				const int32 NumSpheres = OutCollection.NumElements(SpheresGroup);

				OutCollection.AddElements(1, FGeometryCollection::TransformGroup);
				OutCollection.AddElements(NumSpheresFromSDF, SpheresGroup);

				TManagedArray<FTransform>& Transform = OutCollection.AddAttribute<FTransform>("Transform", FGeometryCollection::TransformGroup);
				TManagedArray<FString>& BoneName = OutCollection.AddAttribute<FString>("BoneName", FGeometryCollection::TransformGroup);
				TManagedArray<FLinearColor>& BoneColor = OutCollection.AddAttribute<FLinearColor>("BoneColor", FGeometryCollection::TransformGroup);
				TManagedArray<int32>& Parent = OutCollection.AddAttribute<int32>("Parent", FGeometryCollection::TransformGroup);
				TManagedArray<TSet<int32>>& Children = OutCollection.AddAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);
				TManagedArray<int32>& TransformToSphereIndex = OutCollection.AddAttribute<int32>("TransformToSphereIndex", FGeometryCollection::TransformGroup);
				TManagedArray<int32>& BoneMap = OutCollection.AddAttribute<int32>("BoneMap", SpheresGroup);
				TManagedArray<int32>& TransformIndex = OutCollection.AddAttribute<int32>("TransformIndex", SpheresGroup);
				TManagedArray<FVector3f>& Center = OutCollection.AddAttribute<FVector3f>("Center", SpheresGroup);
				TManagedArray<float>& Radius = OutCollection.AddAttribute<float>("Radius", SpheresGroup);

				Transform[NumTransforms] = FTransform::Identity;
				BoneName[NumTransforms] = FString(TEXT("Spheres"));
				BoneColor[NumTransforms] = FLinearColor(0.02f, 0.01f, 0.1f, 1.0f);
				Parent[NumTransforms] = -1;
				Children[NumTransforms] = TSet<int32>();
				TransformToSphereIndex[NumTransforms] = NumSpheres;

				TArray<FSphere> OutSpheres; OutSpheres.AddUninitialized(NumSpheresFromSDF);

				for (int32 Idx = 0; Idx < NumSpheresFromSDF; ++Idx)
				{
					Center[NumSpheres + Idx] = (FVector3f)OutSphereCenters[Idx];
					Radius[NumSpheres + Idx] = OutSphereRadii[Idx];

					OutSpheres[Idx].Center = OutSphereCenters[Idx];
					OutSpheres[Idx].W = OutSphereRadii[Idx];
				}

				TransformIndex[NumSpheres] = NumTransforms;

				SetValue(Context, MoveTemp(OutCollection), &Collection);
				SetValue(Context, MoveTemp(OutSpheres), &Spheres);

				return;
			}
		}

		SetValue(Context, FManagedArrayCollection(), &Collection);
		SetValue(Context, TArray<FSphere>(), &Spheres);
	}
}

/* --------------------------------------------------------------------------------------------- */

FVolumeToSpheresDataflowNode_v2::FVolumeToSpheresDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&FloatVolume);
	RegisterInputConnection(&MinSphereCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxSphereCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&IsoValue).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&InstanceCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RandomSeed).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Spread).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinNumberOfPointsPerVoxel).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxNumberOfPointsPerVoxel).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&Spheres);
}

void FVolumeToSpheresDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) || Out->IsA(&Spheres))
	{
		if (IsConnected(&FloatVolume))
		{
			const FDataflowFloatVolume& InFloatVolume = GetValue(Context, &FloatVolume);
			const int32 InMinSphereCount = GetValue(Context, &MinSphereCount);
			const int32 InMaxSphereCount = GetValue(Context, &MaxSphereCount);
			const float InMinRadius = GetValue(Context, &MinRadius);
			const float InMaxRadius = GetValue(Context, &MaxRadius);
			const float InIsoValue = GetValue(Context, &IsoValue);
			const int32 InInstanceCount = GetValue(Context, &InstanceCount);
			const int32 InRandomSeed = GetValue(Context, &RandomSeed);
			const float InSpread = GetValue(Context, &Spread);
			const float InMinNumberOfPointsPerVoxel = GetValue(Context, &MinNumberOfPointsPerVoxel);
			const float InMaxNumberOfPointsPerVoxel = GetValue(Context, &MaxNumberOfPointsPerVoxel);

			TArray<FVector> OutSphereCenters;
			TArray<float> OutSphereRadii;

			// Compute sphere packing
			InFloatVolume.VolumeToSpheresImproved(
				InMinSphereCount,
				InMaxSphereCount,
				bOverlapping,
				InMinRadius,
				InMaxRadius,
				InIsoValue,
				InInstanceCount,
				InRandomSeed,
				ScatterType,
				InSpread,
				InMinNumberOfPointsPerVoxel,
				InMaxNumberOfPointsPerVoxel,
				OutSphereCenters,
				OutSphereRadii);

			// Add spheres to OutCollection
			FManagedArrayCollection OutCollection;

			const int32 NumSpheresFromSDF = OutSphereCenters.Num();

			if (NumSpheresFromSDF > 0)
			{
				// Add new element to groups
				static const FName SpheresGroup = "Spheres";

				const int32 NumTransforms = OutCollection.NumElements(FGeometryCollection::TransformGroup);
				const int32 NumSpheres = OutCollection.NumElements(SpheresGroup);

				OutCollection.AddElements(1, FGeometryCollection::TransformGroup);
				OutCollection.AddElements(NumSpheresFromSDF, SpheresGroup);

				TManagedArray<FTransform>& Transform = OutCollection.AddAttribute<FTransform>("Transform", FGeometryCollection::TransformGroup);
				TManagedArray<FString>& BoneName = OutCollection.AddAttribute<FString>("BoneName", FGeometryCollection::TransformGroup);
				TManagedArray<FLinearColor>& BoneColor = OutCollection.AddAttribute<FLinearColor>("BoneColor", FGeometryCollection::TransformGroup);
				TManagedArray<int32>& Parent = OutCollection.AddAttribute<int32>("Parent", FGeometryCollection::TransformGroup);
				TManagedArray<TSet<int32>>& Children = OutCollection.AddAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);
				TManagedArray<int32>& TransformToSphereIndex = OutCollection.AddAttribute<int32>("TransformToSphereIndex", FGeometryCollection::TransformGroup);
				TManagedArray<int32>& BoneMap = OutCollection.AddAttribute<int32>("BoneMap", SpheresGroup);
				TManagedArray<int32>& TransformIndex = OutCollection.AddAttribute<int32>("TransformIndex", SpheresGroup);
				TManagedArray<FVector3f>& Center = OutCollection.AddAttribute<FVector3f>("Center", SpheresGroup);
				TManagedArray<float>& Radius = OutCollection.AddAttribute<float>("Radius", SpheresGroup);

				Transform[NumTransforms] = FTransform::Identity;
				BoneName[NumTransforms] = FString(TEXT("Spheres"));
				BoneColor[NumTransforms] = FLinearColor(0.02f, 0.01f, 0.1f, 1.0f);
				Parent[NumTransforms] = -1;
				Children[NumTransforms] = TSet<int32>();
				TransformToSphereIndex[NumTransforms] = NumSpheres;

				TArray<FSphere> OutSpheres; OutSpheres.AddUninitialized(NumSpheresFromSDF);

				for (int32 Idx = 0; Idx < NumSpheresFromSDF; ++Idx)
				{
					Center[NumSpheres + Idx] = (FVector3f)OutSphereCenters[Idx];
					Radius[NumSpheres + Idx] = OutSphereRadii[Idx];

					OutSpheres[Idx].Center = OutSphereCenters[Idx];
					OutSpheres[Idx].W = OutSphereRadii[Idx];
				}

				TransformIndex[NumSpheres] = NumTransforms;

				SetValue(Context, MoveTemp(OutCollection), &Collection);
				SetValue(Context, MoveTemp(OutSpheres), &Spheres);

				return;
			}
		}

		SetValue(Context, FManagedArrayCollection(), &Collection);
		SetValue(Context, TArray<FSphere>(), &Spheres);
	}
}

/* --------------------------------------------------------------------------------------------- */

FConvertVolumeDataflowNode::FConvertVolumeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&FloatVolume);
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&FloatVolume, &FloatVolume);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Shaded;
	DebugDrawRenderSettings.bTranslucent = false;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 1.5f;
}

void FConvertVolumeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FloatVolume))
	{
		if (IsConnected(&FloatVolume))
		{
			if (ConvertTo == EDataflowVolumeConvertSDFTo::Volume)
			{
				const FDataflowFloatVolume& InFloatVolume = GetValue(Context, &FloatVolume);

				if (!(GridClass == EDataflowVolumeConvertSDFGridClass::NoChange &&
					GridType == EDataflowVolumeConvertSDFGridType::NoChange))
				{
					if (GridClass == EDataflowVolumeConvertSDFGridClass::SDFToFog)
					{
						FDataflowFloatVolume OutVolume = InFloatVolume.CovertSDFToFogVolume(
							bPruneTolerance,
							Tolerance,
							bFloodFillOutput,
							bActivateInteriror);

						SetValue(Context, MoveTemp(OutVolume), &FloatVolume);
					}
					else if (GridClass == EDataflowVolumeConvertSDFGridClass::FogToSDF)
					{
						FDataflowFloatVolume OutVolume = InFloatVolume.CovertFogVolumeToSDF(
							FogIsoValue,
							bPruneTolerance,
							Tolerance,
							bFloodFillOutput,
							bActivateInteriror);

						SetValue(Context, MoveTemp(OutVolume), &FloatVolume);
					}
				}
				else
				{
					SafeForwardInput(Context, &FloatVolume, &FloatVolume);
				}

				SetValue(Context, FManagedArrayCollection(), &Collection);
			}
			else if (ConvertTo == EDataflowVolumeConvertSDFTo::Collection)
			{
				SetValue(Context, FDataflowFloatVolume(), &FloatVolume);
			}

			return;
		}

		SetValue(Context, FDataflowFloatVolume(), &FloatVolume);
	}
	else if (Out->IsA(&Collection))
	{
		if (IsConnected(&FloatVolume))
		{
			if (ConvertTo == EDataflowVolumeConvertSDFTo::Collection)
			{

				const FDataflowFloatVolume& InFloatVolume = GetValue(Context, &FloatVolume);

				FMeshDescription OutMeshDescription;

				InFloatVolume.ConvertVolumeToMeshDescription(
					IsoValue,
					Adaptivity,
					OutMeshDescription);

				FManagedArrayCollection OutCollection = FManagedArrayCollection();

				if (OutMeshDescription.Vertices().Num() > 0)
				{
					// Convert MeshDescription to FManagedArrayCollection
					FGeometryCollection NewGeometryCollection = FGeometryCollection();
					FGeometryCollectionEngineConversion::AppendMeshDescription(&OutMeshDescription, FString(TEXT("TEST")), 0, FTransform().Identity, &NewGeometryCollection);

					NewGeometryCollection.CopyTo(&OutCollection);
				}

				SetValue(Context, MoveTemp(OutCollection), &Collection);
				SafeForwardInput(Context, &FloatVolume, &FloatVolume);

				return;
			}
			else if (ConvertTo == EDataflowVolumeConvertSDFTo::Volume)
			{
				SetValue(Context, FManagedArrayCollection(), &Collection);
			}
		}
	}
}

/* --------------------------------------------------------------------------------------------- */

FUniformVolumeScatterDataflowNode::FUniformVolumeScatterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&FloatVolume);
	RegisterInputConnection(&MinNumberOfPoints).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxNumberOfPoints).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RandomSeed).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&IsoValue).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Spread).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Points);
}

void FUniformVolumeScatterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Points))
	{
		if (IsConnected(&FloatVolume))
		{
			const FDataflowFloatVolume& InFloatVolume = GetValue(Context, &FloatVolume);
			const int32 InMinNumberOfPoints = GetValue(Context, &MinNumberOfPoints);
			const int32 InMaxNumberOfPoints = GetValue(Context, &MaxNumberOfPoints);
			const int32 InRandomSeed = GetValue(Context, &RandomSeed);
			const float InIsoValue = GetValue(Context, &IsoValue);
			const float InSpread = GetValue(Context, &Spread);

			TArray<FVector> OutPoints;

			// Compute point scatter in volume
			InFloatVolume.UniformVolumeScatter(
				InMinNumberOfPoints,
				InMaxNumberOfPoints,
				InRandomSeed,
				InIsoValue,
				InSpread,
				OutPoints);

			SetValue(Context, MoveTemp(OutPoints), &Points);

			return;
		}

		SetValue(Context, TArray<FVector>(), &Points);
	}
}

/* --------------------------------------------------------------------------------------------- */

FDenseUniformVolumeScatterDataflowNode::FDenseUniformVolumeScatterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&FloatVolume);
	RegisterInputConnection(&MinNumberOfPointsPerVoxel).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxNumberOfPointsPerVoxel).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RandomSeed).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&IsoValue).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Spread).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Points);
}

void FDenseUniformVolumeScatterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Points))
	{
		if (IsConnected(&FloatVolume))
		{
			const FDataflowFloatVolume& InFloatVolume = GetValue(Context, &FloatVolume);
			const float InMinNumberOfPointsPerVoxel = GetValue(Context, &MinNumberOfPointsPerVoxel);
			const float InMaxNumberOfPointsPerVoxel = GetValue(Context, &MaxNumberOfPointsPerVoxel);
			const int32 InRandomSeed = GetValue(Context, &RandomSeed);
			const float InIsoValue = GetValue(Context, &IsoValue);
			const float InSpread = GetValue(Context, &Spread);

			TArray<FVector> OutPoints;

			// Compute point scatter in volume
			InFloatVolume.DenseUniformVolumeScatter(
				InMinNumberOfPointsPerVoxel,
				InMaxNumberOfPointsPerVoxel,
				InRandomSeed,
				InIsoValue,
				InSpread,
				OutPoints);

			SetValue(Context, MoveTemp(OutPoints), &Points);

			return;
		}

		SetValue(Context, TArray<FVector>(), &Points);
	}
}

/* --------------------------------------------------------------------------------------------- */

FNonUniformVolumeScatterDataflowNode::FNonUniformVolumeScatterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&FloatVolume);
	RegisterInputConnection(&MinNumberOfPointsPerVoxel).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxNumberOfPointsPerVoxel).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RandomSeed).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&IsoValue).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Spread).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Points);
}

void FNonUniformVolumeScatterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Points))
	{
		if (IsConnected(&FloatVolume))
		{
			const FDataflowFloatVolume& InFloatVolume = GetValue(Context, &FloatVolume);
			const float InMinNumberOfPointsPerVoxel = GetValue(Context, &MinNumberOfPointsPerVoxel);
			const float InMaxNumberOfPointsPerVoxel = GetValue(Context, &MaxNumberOfPointsPerVoxel);
			const int32 InRandomSeed = GetValue(Context, &RandomSeed);
			const float InIsoValue = GetValue(Context, &IsoValue);
			const float InSpread = GetValue(Context, &Spread);

			TArray<FVector> OutPoints;

			// Compute point scatter in volume
			InFloatVolume.NonUniformVolumeScatter(
				InMinNumberOfPointsPerVoxel,
				InMaxNumberOfPointsPerVoxel,
				InRandomSeed,
				InIsoValue,
				InSpread,
				OutPoints);

			SetValue(Context, MoveTemp(OutPoints), &Points);

			return;
		}

		SetValue(Context, TArray<FVector>(), &Points);
	}
}

/* --------------------------------------------------------------------------------------------- */

FVolumeSampleToPointsDataflowNode::FVolumeSampleToPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Volume);
	RegisterInputConnection(&SamplePoints);
	RegisterOutputConnection(&OutArray);
	RegisterOutputConnection(&PointsCollection);

	SetOutTypeFromSampleType();
}

void FVolumeSampleToPointsDataflowNode::SetOutTypeFromSampleType()
{
	if (FDataflowOutput* Output = FindOutput(&OutArray))
	{
		if (!Output->IsConnected())
		{
			Output->UnlockType();

			switch (SampleType)
			{
			case EDataflowVolumeSampleType::Float:
				SetOutputConcreteType<TArray<float>>(&OutArray);
				break;
			case EDataflowVolumeSampleType::Int:
				SetOutputConcreteType<TArray<int32>>(&OutArray);
				break;
			case EDataflowVolumeSampleType::FloatVector:
				SetOutputConcreteType<TArray<FVector>>(&OutArray);
				break;
			default:
				break;
			}

			Output->LockType();
		}
		else
		{
			ensure(false);
		}
	}
}

void FVolumeSampleToPointsDataflowNode::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FVolumeSampleToPointsDataflowNode, SampleType))
	{
		SetOutTypeFromSampleType();
	}
}

void FVolumeSampleToPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&OutArray) || Out->IsA(&PointsCollection))
	{
		const FDataflowOutput* OutArrayOutput = FindOutput(&OutArray);

		if (IsConnected(&Volume) && IsConnected(&SamplePoints))
		{
			// GetValue() for AnyType returns the storage type
			const FDataflowVolume& InVolume = GetValue(Context, &Volume);
			const TArray<FVector4>& InSamplePoints = GetValue(Context, &SamplePoints);
			
			FManagedArrayCollection OutPointsCollection = FManagedArrayCollection();
			OutPointsCollection.AddGroup(FGeometryCollection::VerticesGroup);

			const int32 NumPoints = InSamplePoints.Num();
			TArray<FVector> Points;
			Points.AddUninitialized(NumPoints);

			for (int32 Idx = 0; Idx < NumPoints; ++Idx)
			{
				Points[Idx] = FVector(InSamplePoints[Idx]);
			}

			GeometryCollection::Facades::FPointsCollectionFacade PointsCollectionFacade(OutPointsCollection);

			if (SampleType == EDataflowVolumeSampleType::Float)
			{
				if (const FDataflowFloatVolume* InFloatVolume = InVolume.Cast<FDataflowFloatVolume>())
				{
					TArray<float> OutValues;
					OutValues.AddUninitialized(NumPoints);

					InFloatVolume->VolumeSample(Points, OutValues);

					PointsCollectionFacade.AddPointsWithFloatAttribute(Points, FName(*Attribute), OutValues);
			
					SetValue(Context, MoveTemp(OutPointsCollection), &PointsCollection);

					if (OutArrayOutput)
					{ 
						OutArrayOutput->SetValue<TArray<float>>(MoveTemp(OutValues), Context);
					}

					return;
				}
				else
				{
//					Context.Error(TEXT("The connected volume is not a FloatVolume type"), this, Out);
				}
			}
			else if (SampleType == EDataflowVolumeSampleType::Int)
			{
				// Input volume is an IntVolume
				if (const FDataflowIntVolume* InIntVolume = InVolume.Cast<FDataflowIntVolume>())
				{
					TArray<int32> OutValues;
					OutValues.AddUninitialized(Points.Num());

					InIntVolume->VolumeSample(Points, OutValues);

					PointsCollectionFacade.AddPointsWithIntAttribute(Points, FName(*Attribute), OutValues);

					SetValue(Context, MoveTemp(OutPointsCollection), &PointsCollection);

					if (OutArrayOutput)
					{
						OutArrayOutput->SetValue<TArray<int32>>(MoveTemp(OutValues), Context);
					}

					return;
				}
				else
				{
//					Context.Error(TEXT("The connected volume is not an IntVolume type"), this, Out);
				}
			}
			else if (SampleType == EDataflowVolumeSampleType::FloatVector)
			{
				// Input volume is a FloatVectorVolume
				if (const FDataflowFloatVectorVolume* InFloatVectorVolume = InVolume.Cast<FDataflowFloatVectorVolume>())
				{
					TArray<FVector> OutValues;
					OutValues.AddUninitialized(Points.Num());

					InFloatVectorVolume->VolumeSample(Points, OutValues);

					PointsCollectionFacade.AddPointsWithVectorAttribute(Points, FName(*Attribute), OutValues);

					SetValue(Context, MoveTemp(OutPointsCollection), &PointsCollection);

					if (OutArrayOutput)
					{
						OutArrayOutput->SetValue<TArray<FVector>>(MoveTemp(OutValues), Context);
					}

					return;
				}
				else
				{
//					Context.Error(TEXT("The connected volume is not an FloatVectorVolume type"), this, Out);
				}
			}
		}

		if (OutArrayOutput)
		{
			if (SampleType == EDataflowVolumeSampleType::Float)
			{
				OutArrayOutput->SetValue<TArray<float>>(TArray<float>(), Context);
			}
			else if (SampleType == EDataflowVolumeSampleType::Int)
			{
				OutArrayOutput->SetValue<TArray<int32>>(TArray<int32>(), Context);
			}
			else if (SampleType == EDataflowVolumeSampleType::FloatVector)
			{
				OutArrayOutput->SetValue<TArray<FVector>>(TArray<FVector>(), Context);
			}

			SetValue(Context, FManagedArrayCollection(), &PointsCollection);
		}
	}
}

/* --------------------------------------------------------------------------------------------- */

namespace UE::Dataflow::Volume
{
	static void AdjustBoundingBox(FBox& VolumeBoundingBox, FVector& Extent, const float VoxelSize, const float Scale)
	{

		const FVector NewMin = VolumeBoundingBox.GetCenter() - (VolumeBoundingBox.GetCenter() - VolumeBoundingBox.Min) * Scale;
		const FVector NewMax = VolumeBoundingBox.GetCenter() + (VolumeBoundingBox.Max - VolumeBoundingBox.GetCenter()) * Scale;

		VolumeBoundingBox.Min = NewMin;
		VolumeBoundingBox.Max = NewMax;

		Extent = 2.f * VolumeBoundingBox.GetExtent();

		float X = Extent.X / VoxelSize; Extent.X = ceil(X) * VoxelSize;
		float Y = Extent.Y / VoxelSize; Extent.Y = ceil(Y) * VoxelSize;
		float Z = Extent.Z / VoxelSize; Extent.Z = ceil(Z) * VoxelSize;

		VolumeBoundingBox.Min = VolumeBoundingBox.GetCenter() - 0.5f * Extent;
		VolumeBoundingBox.Max = VolumeBoundingBox.GetCenter() + 0.5f * Extent;
	}

	static TArray<FVector> GetSlicePoints(const FBox& VolumeBoundingBox,
		const EDataflowVolumeSlicePlane Plane,
		const float VoxelSize,
		const float Offset)
	{
		TArray<FVector> Points;
		int32 NumPoints;

		FVector Extent = 2.f * VolumeBoundingBox.GetExtent();

		FVector Min = VolumeBoundingBox.Min;
		FVector Max = VolumeBoundingBox.Max;

		if (Plane == EDataflowVolumeSlicePlane::XYPlane)
		{
			const int32 NumPointsX = int32(Extent.X / VoxelSize);
			const int32 NumPointsY = int32(Extent.Y / VoxelSize);
			NumPoints = NumPointsX * NumPointsY;

			Points.AddUninitialized(NumPoints);

			int32 Idx = 0;
			for (int32 IdxX = 0; IdxX < NumPointsX; ++IdxX)
			{
				for (int32 IdxY = 0; IdxY < NumPointsY; ++IdxY)
				{
					Points[Idx].X = Min.X + float(IdxX) * VoxelSize;
					Points[Idx].Y = Min.Y + float(IdxY) * VoxelSize;
					Points[Idx++].Z = Max.Z - .5f * (1.f - Offset) * (Max.Z - Min.Z);
				}
			}
		}
		else if (Plane == EDataflowVolumeSlicePlane::YZPlane)
		{
			const int32 NumPointsY = int32(Extent.Y / VoxelSize);
			const int32 NumPointsZ = int32(Extent.Z / VoxelSize);
			NumPoints = NumPointsY * NumPointsZ;

			Points.AddUninitialized(NumPoints);

			int32 Idx = 0;
			for (int32 IdxY = 0; IdxY < NumPointsY; ++IdxY)
			{
				for (int32 IdxZ = 0; IdxZ < NumPointsZ; ++IdxZ)
				{
					Points[Idx].X = Max.X - .5f * (1.f - Offset) * (Max.X - Min.X);
					Points[Idx].Y = Min.Y + float(IdxY) * VoxelSize;
					Points[Idx++].Z = Min.Z + float(IdxZ) * VoxelSize;
				}
			}
		}
		else if (Plane == EDataflowVolumeSlicePlane::ZXPlane)
		{
			const int32 NumPointsZ = int32(Extent.Z / VoxelSize);
			const int32 NumPointsX = int32(Extent.X / VoxelSize);
			NumPoints = NumPointsZ * NumPointsX;

			Points.AddUninitialized(NumPoints);

			int32 Idx = 0;
			for (int32 IdxZ = 0; IdxZ < NumPointsZ; ++IdxZ)
			{
				for (int32 IdxX = 0; IdxX < NumPointsX; ++IdxX)
				{
					Points[Idx].X = Min.X + float(IdxX) * VoxelSize;
					Points[Idx].Y = Max.Y - .5f * (1.f - Offset) * (Max.Y - Min.Y);
					Points[Idx++].Z = Min.Z + float(IdxZ) * VoxelSize;
				}
			}
		}

		return Points;
	}

	static TArray< FLinearColor> GetInfraRedColorSpectrum()
	{
		const int32 NumColors = 7;
		TArray<FLinearColor> Spectrum;
		Spectrum.AddUninitialized(NumColors);

		Spectrum[6] = FLinearColor::Red;
		Spectrum[5] = FLinearColor(FColor(255, 128, 0, 1)); // Orange
		Spectrum[4] = FLinearColor::Yellow;
		Spectrum[3] = FLinearColor::Green;
		Spectrum[2] = FLinearColor::Blue;
		Spectrum[1] = FLinearColor(FColor(64, 0, 128, 1)); // Indigo
		Spectrum[0] = FLinearColor(FColor(128, 0, 128, 1)); // Violet

		return Spectrum;
	}
}

FVolumeSliceDataflowNode::FVolumeSliceDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static const FName TypeDependencyGroup("Main");
	RegisterInputConnection(&Volume)
		.SetTypeDependencyGroup(TypeDependencyGroup);
	RegisterOutputConnection(&Volume, &Volume)
		.SetTypeDependencyGroup(TypeDependencyGroup);

	// Set ColorRamp
	const int32 NumColors = 7;
	TArray<FLinearColor> Spectrum;
	Spectrum.AddUninitialized(NumColors);

	Spectrum[6] = FLinearColor::Red;
	Spectrum[5] = FLinearColor(FColor(255, 128, 0, 1)); // Orange
	Spectrum[4] = FLinearColor::Yellow;
	Spectrum[3] = FLinearColor::Green;
	Spectrum[2] = FLinearColor::Blue;
	Spectrum[1] = FLinearColor(FColor(64, 0, 128, 1)); // Indigo
	Spectrum[0] = FLinearColor(FColor(128, 0, 128, 1)); // Violet

	for (int32 Idx = 0; Idx < NumColors; ++Idx)
	{
		LinearColorRamp.GetCurves()[0].CurveToEdit->AddKey((float)Idx * 1.f / ((float)NumColors - 1.f), Spectrum[Idx].R);
		LinearColorRamp.GetCurves()[1].CurveToEdit->AddKey((float)Idx * 1.f / ((float)NumColors - 1.f), Spectrum[Idx].G);
		LinearColorRamp.GetCurves()[2].CurveToEdit->AddKey((float)Idx * 1.f / ((float)NumColors - 1.f), Spectrum[Idx].B);
	}
	LinearColorRamp.GetCurves()[3].CurveToEdit->AddKey(0.f, 1.f);

	//
	LinearColorRamp.OnColorCurveChangedDelegate.AddRaw(this, &FVolumeSliceDataflowNode::OnColorCurveChanged);
}

void FVolumeSliceDataflowNode::OnColorCurveChanged(TArray<FRichCurve*> Curves)
{
	Invalidate();
}

void FVolumeSliceDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Volume))
	{
		ForwardInput(Context, &Volume, &Volume);
	}
}

void FVolumeSliceDataflowNode::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent)
{
#if WITH_EDITOR
	const FName PropertyName = InPropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FVolumeSliceDataflowNode, ColorRampType))
	{
		TArray<FLinearColor> Spectrum;
		int32 NumColors = 0;

		if (ColorRampType == EDataflowVolumeSliceRamp::InfraRed)
		{
			NumColors = 7;
			Spectrum.AddUninitialized(NumColors);

			Spectrum[0] = FLinearColor(FColor(128, 0, 128, 1)); // Violet
			Spectrum[1] = FLinearColor(FColor(64, 0, 128, 1)); // Indigo
			Spectrum[2] = FLinearColor::Blue;
			Spectrum[3] = FLinearColor::Green;
			Spectrum[4] = FLinearColor::Yellow;
			Spectrum[5] = FLinearColor(FColor(255, 128, 0, 1)); // Orange
			Spectrum[6] = FLinearColor::Red;
		}
		else if (ColorRampType == EDataflowVolumeSliceRamp::WhiteToRed)
		{
			NumColors = 2;
			Spectrum.AddUninitialized(NumColors);

			Spectrum[0] = FLinearColor::White;
			Spectrum[1] = FLinearColor::Red;
		}
		else if (ColorRampType == EDataflowVolumeSliceRamp::BlackBody)
		{
			NumColors = 4;
			Spectrum.AddUninitialized(NumColors);

			Spectrum[0] = FLinearColor::White;
			Spectrum[1] = FLinearColor::Yellow;
			Spectrum[2] = FLinearColor::Red;
			Spectrum[3] = FLinearColor::Black;
		}
		else if (ColorRampType == EDataflowVolumeSliceRamp::Grayscale)
		{
			NumColors = 2;
			Spectrum.AddUninitialized(NumColors);

			Spectrum[0] = FLinearColor::Black;
			Spectrum[1] = FLinearColor::White;
		}
		else if (ColorRampType == EDataflowVolumeSliceRamp::Custom)
		{
			NumColors = 2;
			Spectrum.AddUninitialized(NumColors);

			Spectrum[0] = FLinearColor::Red;
			Spectrum[1] = FLinearColor::Blue;
		}

		// Update ColorRamp
		for (int32 Idx = 0; Idx < 4; ++Idx)
		{
			LinearColorRamp.GetCurves()[Idx].CurveToEdit->Reset();
		}

		for (int32 Idx = 0; Idx < NumColors; ++Idx)
		{
			LinearColorRamp.GetCurves()[0].CurveToEdit->AddKey((float)Idx * 1.f / ((float)NumColors - 1.f), Spectrum[Idx].R);
			LinearColorRamp.GetCurves()[1].CurveToEdit->AddKey((float)Idx * 1.f / ((float)NumColors - 1.f), Spectrum[Idx].G);
			LinearColorRamp.GetCurves()[2].CurveToEdit->AddKey((float)Idx * 1.f / ((float)NumColors - 1.f), Spectrum[Idx].B);
		}
		LinearColorRamp.GetCurves()[3].CurveToEdit->AddKey(0.f, 1.f);

	}
#endif
}

#if WITH_EDITOR
bool FVolumeSliceDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FVolumeSliceDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	using namespace UE::Dataflow::Overlay;

	if (DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned)
	{
		if (IsConnected(&Volume))
		{
			if (bVisualize)
			{
				const FDataflowVolume& InVolume = GetValue(Context, &Volume);
				if (const FDataflowFloatVolume* InFloatVolume = InVolume.Cast<FDataflowFloatVolume>())
				{
					const float VoxelSize = InFloatVolume->GetVoxelSize()[0];
					FBox VolumeBoundingBox = InFloatVolume->GetVolumeBoundingBox();

					FVector Extent;
					UE::Dataflow::Volume::AdjustBoundingBox(VolumeBoundingBox, Extent, VoxelSize, BoundingBoxScale);

					TArray<FVector> SamplePoints = UE::Dataflow::Volume::GetSlicePoints(VolumeBoundingBox, Plane, VoxelSize, Offset);

					// Sample points in the volume
					TArray<float> SampledValues;
					SampledValues.AddUninitialized(SamplePoints.Num());

					InFloatVolume->VolumeSample(SamplePoints, SampledValues);

					DataflowRenderingInterface.SetWorldPriority();

					// Draw BoundingBox
					DataflowRenderingInterface.SetColor(FLinearColor::Gray);
					DataflowRenderingInterface.DrawBox(0.5 * Extent, FQuat::Identity, VolumeBoundingBox.GetCenter(), 1.f);

					for (int32 Idx = 0; Idx < SamplePoints.Num(); ++Idx)
					{
						const float SampledValue = FMath::Clamp(SampledValues[Idx], Range.X, Range.Y);
						
						float Progress = ((Range.Y - Range.X) != 0)
							? (1.f - (Range.Y - SampledValue) / (Range.Y - Range.X))
							: 0.0f;
						if (bInvertColorRamp)
						{
							Progress = 1.f - Progress;
						}
						const FLinearColor Color = LinearColorRamp.GetLinearColorValue(Progress);

						DataflowRenderingInterface.SetColor(Color);
						DataflowRenderingInterface.DrawPoint(SamplePoints[Idx]);
					}
				}
				else if (const FDataflowFloatVectorVolume* InFloatVectorVolume = InVolume.Cast<FDataflowFloatVectorVolume>())
				{
					const float VoxelSize = InFloatVectorVolume->GetVoxelSize()[0];
					FBox VolumeBoundingBox = InFloatVectorVolume->GetVolumeBoundingBox();

					FVector Extent;
					UE::Dataflow::Volume::AdjustBoundingBox(VolumeBoundingBox, Extent, VoxelSize, BoundingBoxScale);

					TArray<FVector> SamplePoints = UE::Dataflow::Volume::GetSlicePoints(VolumeBoundingBox, Plane, VoxelSize, Offset);

					// Sample points in the volume
					TArray<FVector> SampledValues;
					SampledValues.AddUninitialized(SamplePoints.Num());

					InFloatVectorVolume->VolumeSample(SamplePoints, SampledValues);

					DataflowRenderingInterface.SetWorldPriority();
					DataflowRenderingInterface.SetPointSize(PointScale);
					DataflowRenderingInterface.SetLineWidth(LineWidth);

					// Draw BoundingBox
					DataflowRenderingInterface.SetColor(FLinearColor::Gray);
					DataflowRenderingInterface.DrawBox(0.5 * Extent, FQuat::Identity, VolumeBoundingBox.GetCenter(), 1.f);

					// Draw values
					if (InFloatVectorVolume->GetVectorType() == "Covariant")
					{
						for (int32 Idx = 0; Idx < SamplePoints.Num(); ++Idx)
						{
							if (SampledValues[Idx].Length() > MinLength)
							{
								DataflowRenderingInterface.SetColor(VectorColor);
								DataflowRenderingInterface.DrawLine(SamplePoints[Idx], SamplePoints[Idx] + VectorScale * SampledValues[Idx]);

								DataflowRenderingInterface.SetColor(PointColor);
								DataflowRenderingInterface.DrawPoint(SamplePoints[Idx]);
							}
						}
					}
				}
			}
		}
	}
}
#endif

/* --------------------------------------------------------------------------------------------- */

FVolumeAnalysisDataflowNode::FVolumeAnalysisDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Volume);
	RegisterInputConnection(&MaskVolume);
	RegisterOutputConnection(&OutVolume);
	RegisterOutputConnection(&MaskVolume, &MaskVolume);

	SetOutTypeFromOperator();
}

void FVolumeAnalysisDataflowNode::SetOutTypeFromOperator()
{
	if (FDataflowOutput* Output = FindOutput(&OutVolume))
	{
		if (!Output->IsConnected())
		{
			Output->UnlockType();

			if (Operator == EDataflowVolumeAnalysisOperator::Gradient ||
				Operator == EDataflowVolumeAnalysisOperator::ClosestPoint ||
				Operator == EDataflowVolumeAnalysisOperator::Curl ||
				Operator == EDataflowVolumeAnalysisOperator::Normalize)
			{
				SetOutputConcreteType<FDataflowFloatVectorVolume>(&OutVolume);
			}
			else if (Operator == EDataflowVolumeAnalysisOperator::Curvature ||
				Operator == EDataflowVolumeAnalysisOperator::Laplacian ||
				Operator == EDataflowVolumeAnalysisOperator::Divergence ||
				Operator == EDataflowVolumeAnalysisOperator::Magnitude)
			{
				SetOutputConcreteType<FDataflowFloatVolume>(&OutVolume);
			}

			Output->LockType();
		}
		else
		{
			ensure(false);
		}
	}
}

void FVolumeAnalysisDataflowNode::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FVolumeAnalysisDataflowNode, Operator))
	{
		SetOutTypeFromOperator();
	}
}

void FVolumeAnalysisDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&OutVolume))
	{
		const FDataflowOutput* VolumeOutput = FindOutput(&OutVolume);
			
		if (VolumeOutput)
		{
			if (IsConnected(&Volume))
			{
				FDataflowBoolVolume* InMaskVolumePtr = nullptr;
				if (IsConnected(&MaskVolume))
				{
					FDataflowBoolVolume InMaskVolume = GetValue(Context, &MaskVolume);
					InMaskVolumePtr = &InMaskVolume;
				}

				const FDataflowVolume& InVolume = GetValue(Context, &Volume);
				if (InVolume.GetNumActiveVoxels(0.f, false))
				{
					if (const FDataflowFloatVolume* InFloatVolume = InVolume.Cast<FDataflowFloatVolume>())
					{
						// Scalar -> Vector
						if (Operator == EDataflowVolumeAnalysisOperator::Gradient)
						{
							FDataflowFloatVectorVolume GradientVolume = InFloatVolume->ComputeGradient(OutputName, CustomName, InMaskVolumePtr);
							VolumeOutput->SetValue<FDataflowFloatVectorVolume>(MoveTemp(GradientVolume), Context);
							
							return;
						}
						// Scalar -> Scalar
						else if (Operator == EDataflowVolumeAnalysisOperator::Curvature)
						{
							FDataflowFloatVolume CurvatureVolume = InFloatVolume->ComputeCurvature(OutputName, CustomName, InMaskVolumePtr);
							VolumeOutput->SetValue<FDataflowFloatVolume>(MoveTemp(CurvatureVolume), Context);

							return;
						}
						// Scalar -> Scalar
						else if (Operator == EDataflowVolumeAnalysisOperator::Laplacian)
						{
							FDataflowFloatVolume LaplacianVolume = InFloatVolume->ComputeLaplacian(OutputName, CustomName, InMaskVolumePtr);
							VolumeOutput->SetValue<FDataflowFloatVolume>(MoveTemp(LaplacianVolume), Context);

							return;
						}
						// Scalar -> Vector
						else if (Operator == EDataflowVolumeAnalysisOperator::ClosestPoint)
						{
							FDataflowFloatVectorVolume ClosestPointVolume = InFloatVolume->ComputeClosestPoint(OutputName, CustomName, InMaskVolumePtr);
							VolumeOutput->SetValue<FDataflowFloatVectorVolume>(MoveTemp(ClosestPointVolume), Context);

							return;
						}
					}
					else if (const FDataflowFloatVectorVolume* InFloatVectorVolume = InVolume.Cast<FDataflowFloatVectorVolume>())
					{
						// Vector -> Scalar
						if (Operator == EDataflowVolumeAnalysisOperator::Divergence)
						{
							FDataflowFloatVolume DivergenceVolume = InFloatVectorVolume->ComputeDivergence(OutputName, CustomName);
							//						FDataflowFloatVolume DivergenceVolume = InFloatVectorVolume->ComputeAnalysis<FDataflowFloatVolume, "Divergence">(OutputName, CustomName, InMaskVolumePtr);
							VolumeOutput->SetValue<FDataflowFloatVolume>(MoveTemp(DivergenceVolume), Context);

							return;
						}
						// Vector -> Vector
						else if (Operator == EDataflowVolumeAnalysisOperator::Curl)
						{
							FDataflowFloatVectorVolume CurlVolume = InFloatVectorVolume->ComputeCurl(OutputName, CustomName);
							//						FDataflowFloatVectorVolume CurlVolume = InFloatVectorVolume->ComputeAnalysis<FDataflowFloatVectorVolume, "Curl">(OutputName, CustomName, InMaskVolumePtr);
							VolumeOutput->SetValue<FDataflowFloatVectorVolume>(MoveTemp(CurlVolume), Context);

							return;
						}
						// Vector -> Scalar
						else if (Operator == EDataflowVolumeAnalysisOperator::Magnitude)
						{
							FDataflowFloatVolume LengthVolume = InFloatVectorVolume->ComputeMagnitude(OutputName, CustomName);
							VolumeOutput->SetValue<FDataflowFloatVolume>(MoveTemp(LengthVolume), Context);

							return;
						}
						// Vector -> Vector
						else if (Operator == EDataflowVolumeAnalysisOperator::Normalize)
						{
							FDataflowFloatVectorVolume NormalVolume = InFloatVectorVolume->ComputeNormalize(OutputName, CustomName);
							VolumeOutput->SetValue<FDataflowFloatVectorVolume>(MoveTemp(NormalVolume), Context);

							return;
						}
					}
				}
			}

			if (Operator == EDataflowVolumeAnalysisOperator::Gradient ||
				Operator == EDataflowVolumeAnalysisOperator::ClosestPoint ||
				Operator == EDataflowVolumeAnalysisOperator::Curl ||
				Operator == EDataflowVolumeAnalysisOperator::Normalize)
			{
				VolumeOutput->SetValue<FDataflowFloatVectorVolume>(FDataflowFloatVectorVolume(), Context);
			}
			else if (Operator == EDataflowVolumeAnalysisOperator::Curvature ||
				Operator == EDataflowVolumeAnalysisOperator::Laplacian ||
				Operator == EDataflowVolumeAnalysisOperator::Divergence ||
				Operator == EDataflowVolumeAnalysisOperator::Magnitude)
			{
				VolumeOutput->SetValue<FDataflowFloatVolume>(FDataflowFloatVolume(), Context);
			}
		}
	}
	else if (Out->IsA(&MaskVolume))
	{
		if (IsConnected(&MaskVolume))
		{
			SafeForwardInput(Context, &MaskVolume, &MaskVolume);
		}
	}
}

/* --------------------------------------------------------------------------------------------- */

FVolumeCombineDataflowNode::FVolumeCombineDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&FloatVolumeA);
	RegisterInputConnection(&FloatVolumeB);
	RegisterInputConnection(&MultiplierA).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MultiplierB).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&FloatVolume);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.bTranslucent = false;
	DebugDrawRenderSettings.Color = FLinearColor(FColor::Cyan);
	DebugDrawRenderSettings.LineWidthMultiplier = 1.5f;
}

void FVolumeCombineDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FloatVolume))
	{
		if (IsConnected(&FloatVolumeA) && IsConnected(&FloatVolumeB))
		{
			FDataflowFloatVolume InFloatVolumeA = GetValue(Context, &FloatVolumeA);
			FDataflowFloatVolume InFloatVolumeB = GetValue(Context, &FloatVolumeB);

			if (bFlipInputs)
			{
				InFloatVolumeA = GetValue(Context, &FloatVolumeB);
				InFloatVolumeB = GetValue(Context, &FloatVolumeA);
			}

			const float InMultiplierA = GetValue(Context, &MultiplierA);
			const float InMultiplierB = GetValue(Context, &MultiplierB);

			FDataflowFloatVolume OutVolume = InFloatVolumeA.VolumeCombine(
				InFloatVolumeB,
				Operation,
				InMultiplierA,
				InMultiplierB,
				Resample,
				Interpolation,
				bPruneDegenerateTiles,
				PruneTol);

			SetValue(Context, MoveTemp(OutVolume), &FloatVolume);
			return;
		}

		SetValue(Context, FDataflowFloatVolume(), &FloatVolume);
	}
}

/* --------------------------------------------------------------------------------------------- */

namespace DataflowVolumeNodes::Private
{
	void UpdateVolumeTextureFromVolume(UVolumeTexture* InVolumeTexture, const FDataflowVolume& InVolume)
	{
		if (!InVolumeTexture)
		{
			return;
		}

#if WITH_EDITOR
		InVolumeTexture->PreEditChange(nullptr);

		if (!InVolume.CreateVolumeTexture(InVolumeTexture))
		{
			return;
		}
		InVolumeTexture->UpdateResource();

		InVolumeTexture->PostEditChange();
#endif
	}
}

FVolumeTextureTerminalNode::FVolumeTextureTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	static const FName MainTypeGroup("Main");

	RegisterInputConnection(&Volume)
		.SetTypeDependencyGroup(MainTypeGroup);

	RegisterInputConnection(&VolumeTextureAsset);

	RegisterOutputConnection(&Volume, &Volume)
		.SetTypeDependencyGroup(MainTypeGroup);
}

void FVolumeTextureTerminalNode::Evaluate(UE::Dataflow::FContext& Context) const
{
	ForwardInput(Context, &Volume, &Volume);
}


void FVolumeTextureTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	UVolumeTexture* AssetToSet = Cast<UVolumeTexture>(Asset.Get());
	if (!AssetToSet)
	{
		AssetToSet = GetValue(Context, &VolumeTextureAsset);
	}

	if (AssetToSet)
	{
		const FDataflowVolume& InVolume = GetValue(Context, &Volume);

		if (InVolume.GetNumActiveVoxels(0.f, false) > 0)
		{
			DataflowVolumeNodes::Private::UpdateVolumeTextureFromVolume(AssetToSet, InVolume);
		}
		else
		{
			Context.Warning(TEXT("Volume doesn't contain data, VolumeTexture will be empty."));
		}
	}
}

/* --------------------------------------------------------------------------------------------- */

FVolumeToSpheresWithRelaxationDataflowNode::FVolumeToSpheresWithRelaxationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&FloatVolume);
	RegisterInputConnection(&PointCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ScatterRandomSeed).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Spread).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RandomSeed).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&IsoValue).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Steps).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&StepScalar).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&DistanceThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Spheres);
}

void FVolumeToSpheresWithRelaxationDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Spheres))
	{
		TArray<FSphere> OutSpheres;

		if (IsConnected(&FloatVolume))
		{
			const FDataflowFloatVolume& InFloatVolume = GetValue(Context, &FloatVolume);
			const int32 InPointCount = GetValue(Context, &PointCount);
			const int32 InScatterRandomSeed = GetValue(Context, &ScatterRandomSeed);
			const float InSpread = GetValue(Context, &Spread);
			const float InMinRadius = GetValue(Context, &MinRadius);
			const float InMaxRadius = GetValue(Context, &MaxRadius);
			const int32 InRandomSeed = GetValue(Context, &RandomSeed);
			const float InIsoValue = GetValue(Context, &IsoValue);
			const int32 InSteps = GetValue(Context, &Steps);
			const float InStepScalar = GetValue(Context, &StepScalar);
			const float InDistanceThreshold = GetValue(Context, &DistanceThreshold);

			// Compute sphere packing
			InFloatVolume.VolumeToSpheresWithRelaxation(
				InPointCount,
				InScatterRandomSeed,
				InMinRadius,
				InMaxRadius,
				InRandomSeed,
				InIsoValue,
				InSpread,
				bRemoveEmbededSpheres,
				bApplyRelaxation,
				InSteps,
				InStepScalar,
				bCorrectAgainstSurface,
				InDistanceThreshold,
				OutSpheres);
		}

		SetValue(Context, MoveTemp(OutSpheres), &Spheres);
	}
}
