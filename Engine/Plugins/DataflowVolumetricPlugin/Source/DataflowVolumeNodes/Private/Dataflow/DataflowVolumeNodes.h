// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowFloatVolume.h"
#include "Dataflow/DataflowFloatVectorVolume.h"
#include "Dataflow/DataflowIntVolume.h"
#include "Dataflow/DataflowBoolVolume.h"
#include "Dataflow/DataflowVolumeAnyType.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowColorRamp.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowTerminalNode.h"
#include "Curves/LinearColorRamp.h"

#include "DataflowVolumeNodes.generated.h"

/**
 * Make a sphere SDF
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FMakeSDFSphereDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeSDFSphereDataflowNode, "MakeSDFSphere", "Volume|Generators", "")

private:
	/** The size of voxels in the output VDB volume */
	UPROPERTY(EditAnywhere, Category = "SDF", meta = (DataflowInput, ClampMin = 0.1f));
	float VoxelSize = 1.f;

	/** Sphere radius */
	UPROPERTY(EditAnywhere, Category = "Sphere", meta = (DataflowInput, ClampMin = 0.1f));
	float Radius = 3.f;

	/** Sphere center */
	UPROPERTY(EditAnywhere, Category = "Sphere", meta = (DataflowInput));
	FVector Center = FVector(0);

	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	UPROPERTY()
	bool bDrawVoxels = true;

	UPROPERTY()
	bool bDrawVoxelCenters = false;

	/** SDF volume */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatVolume FloatVolume;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMakeSDFSphereDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Make a cube SDF
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FMakeSDFCubeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeSDFCubeDataflowNode, "MakeSDFCube", "Volume|Generators", "")

private:
	/** Scale */
	UPROPERTY(EditAnywhere, Category = "Cube", meta = (DataflowInput));
	float Scale = 1.f;

	/** Center */
	UPROPERTY(EditAnywhere, Category = "Cube", meta = (DataflowInput));
	FVector Center = FVector(0.0);

	/** The size of voxels in the output VDB volume */
	UPROPERTY(EditAnywhere, Category = "SDF", meta = (DataflowInput, ClampMin = 0.1f));
	float VoxelSize = 1.f;
	
	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	UPROPERTY()
	bool bDrawVoxels = true;

	UPROPERTY()
	bool bDrawVoxelCenters = false;

	/** SDF volume */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatVolume FloatVolume;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMakeSDFCubeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Make a platonic solid SDF
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FMakeSDFPlatonicSolidDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeSDFPlatonicSolidDataflowNode, "MakeSDFPlatonicSolid", "Volume|Generators", "")

private:
	/** Platonic solid type */
	UPROPERTY(EditAnywhere, Category = "Platonic Solid");
	EDataflowVolumePlatonicSolidType PlatonicSolidType = EDataflowVolumePlatonicSolidType::Tetrahedron;

	/** Scale */
	UPROPERTY(EditAnywhere, Category = "Platonic Solid", meta = (DataflowInput));
	float Scale = 1.f;

	/** Center */
	UPROPERTY(EditAnywhere, Category = "Platonic Solid", meta = (DataflowInput));
	FVector Center = FVector(0.0);

	/** The size of voxels in the output VDB volume */
	UPROPERTY(EditAnywhere, Category = "SDF", meta = (DataflowInput, ClampMin = 0.1f));
	float VoxelSize = 1.f;

	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	UPROPERTY()
	bool bDrawVoxels = true;

	UPROPERTY()
	bool bDrawVoxelCenters = false;

	/** SDF volume */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatVolume FloatVolume;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMakeSDFPlatonicSolidDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Outputs info of a volume
 * Grid(s), resolution, type, voxel count, voxel size
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FVolumeInfoDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVolumeInfoDataflowNode, "VolumeInfo", "Volume|Utilities", "")

private:
	/** Volume */
	UPROPERTY(meta = (DataflowInput))
	FDataflowFloatVolume FloatVolume;

	/** Formatted string containing the groups and attributes */
	UPROPERTY(meta = (DataflowOutput))
	FString String;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FVolumeInfoDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Convert a Collection to a volume
 *
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FCollectionToVolumeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionToVolumeDataflowNode, "CollectionToVolume", "Volume|Utilities", "")

private:
	/** Collection to convert to volume */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic));
	FManagedArrayCollection Collection;

	/** 
	 * Optional reference volume, to use this grid's orientation and voxel size to create the new one.
	 * If the reference VDB is a Level set then match the narrow band width as well.
	 */
	UPROPERTY(meta = (DataflowInput));
	FDataflowVolumeTypes ReferenceVolume;

	/** The size of voxels in the SDF */
	UPROPERTY(EditAnywhere, Category = "SDF", meta = (DataflowInput, ClampMin = 0.1f));
	float VoxelSize = 1.f;

	/** Name of the SDF grid in the VDB */
	UPROPERTY(EditAnywhere, Category = "SDF");
	EDataflowVolumeOutputType OutputType = EDataflowVolumeOutputType::SDF;

	/** Name of the SDF grid in the VDB */
	UPROPERTY(EditAnywhere, Category = "SDF");
	FString GridName = FString(TEXT("LevelSet"));

	/** Use world space units for narrow band */
	UPROPERTY(EditAnywhere, Category = "SDF");
	bool bUseWorldSpaceUnits = false;

	/** */
	UPROPERTY(EditAnywhere, Category = "SDF", meta = (ClampMin = 0.00001f, ClampMax = 10.f, EditCondition = "bUseWorldSpaceUnits == true", EditConditionHides));
	float ExteriorBand = 1.f;

	/** */
	UPROPERTY(EditAnywhere, Category = "SDF", meta = (ClampMin = 0.00001f, ClampMax = 10.f, EditCondition = "bUseWorldSpaceUnits == true && bFillInterior == false", EditConditionHides));
	float InteriorBand = 1.f;

	/** */
	UPROPERTY(EditAnywhere, Category = "SDF", meta = (ClampMin = 1, ClampMax = 10, EditCondition = "bUseWorldSpaceUnits == false", EditConditionHides));
	int32 ExteriorBandVoxels = 3;

	/** */
	UPROPERTY(EditAnywhere, Category = "SDF", meta = (ClampMin = 1, ClampMax = 10, EditCondition = "bUseWorldSpaceUnits == false && bFillInterior == false", EditConditionHides));
	int32 InteriorBandVoxels = 3;

	/** */
	UPROPERTY(EditAnywhere, Category = "SDF");
	bool bFillInterior = false;

	/**  */
	UPROPERTY(EditAnywhere, Category = "SDF");
	bool bPreserveHoles = false;

	/** IsoValue specifying the surface for ClosestSurfacePointVolume */
	UPROPERTY(EditAnywhere, Category = "Closest Surface Point", meta = (DataflowInput));
	float IsoValue = 0.f;

	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	UPROPERTY()
	bool bDrawVoxels = false;

	UPROPERTY()
	bool bDrawVoxelCenters = true;

	/** Float volume */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatVolume FloatVolume;

	/** Every voxel stores the closest face index */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "FaceIndex Volume"))
	FDataflowIntVolume FaceIndexVolume;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FCollectionToVolumeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Fill an SDF/Fog volume with spheres, using the original algorithm
 *
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FVolumeToSpheresDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVolumeToSpheresDataflowNode, "VolumeToSpheres", "Volume|Utilities", "")

private:
	/** Float volume */
	UPROPERTY(meta = (DataflowInput))
	FDataflowFloatVolume FloatVolume;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, DisplayName = "Minimum Sphere Count"));
	int32 MinSphereCount = 1;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, DisplayName = "Maximum Sphere Count"));
	int32 MaxSphereCount = 50;

	UPROPERTY(EditAnywhere, Category = "Spheres");
	bool bOverlapping = false;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, DisplayName = "Minimum Radius", ClampMin = 0.001f));
	float MinRadius = 1.f;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, DisplayName = "Maximum Radius", ClampMin = 0.001f));
	float MaxRadius = 1000.f;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput));
	float IsoValue = 0.f;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, ClampMin = 1));
	int32 InstanceCount = 10000;

	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	UPROPERTY()
	bool bDrawSpheres = true;

	UPROPERTY()
	bool bDrawSphereCenters = false;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FSphere> Spheres;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FVolumeToSpheresDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Fill an SDF/Fog volume with spheres, using the improved algorithm
 *
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FVolumeToSpheresDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVolumeToSpheresDataflowNode_v2, "VolumeToSpheres", "Volume|Utilities", "")

private:
	/** Float volume */
	UPROPERTY(meta = (DataflowInput))
	FDataflowFloatVolume FloatVolume;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, DisplayName = "Minimum Sphere Count"));
	int32 MinSphereCount = 1;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, DisplayName = "Maximum Sphere Count"));
	int32 MaxSphereCount = 50;

	UPROPERTY(EditAnywhere, Category = "Spheres");
	bool bOverlapping = false;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, DisplayName = "Minimum Radius", ClampMin = 0.001f));
	float MinRadius = 1.f;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, DisplayName = "Maximum Radius", ClampMin = 0.001f));
	float MaxRadius = 1000.f;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput));
	float IsoValue = 0.f;

	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, ClampMin = 1));
	int32 InstanceCount = 10000;

	UPROPERTY(EditAnywhere, Category = "Scatter")
	EDataflowVolumeScatterType ScatterType = EDataflowVolumeScatterType::Uniform;

	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput));
	int32 RandomSeed = 0;

	/** */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = 0, ClampMax = 1));
	float Spread = 1.f;

	/** Minimum for the random range */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = 0, EditCondition = "ScatterType != EDataflowVolumeScatterType::Uniform", EditConditionHides));
	float MinNumberOfPointsPerVoxel = 1.f;

	/** Maximum for the random range */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = 0, EditCondition = "ScatterType != EDataflowVolumeScatterType::Uniform", EditConditionHides));
	float MaxNumberOfPointsPerVoxel = 1.f;

	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	UPROPERTY()
	bool bDrawSpheres = true;

	UPROPERTY()
	bool bDrawSphereCenters = false;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FSphere> Spheres;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FVolumeToSpheresDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Convert a volume to Volume/Polygons
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FConvertVolumeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertVolumeDataflowNode, "ConvertVolume", "Volume|Utilities", "")

private:
	/** Float volume */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "FloatVolume", DataflowIntrinsic))
	FDataflowFloatVolume FloatVolume;

	UPROPERTY(EditAnywhere, Category = "Convert", meta = (DisplayName = "Convert To"));
	EDataflowVolumeConvertSDFTo ConvertTo = EDataflowVolumeConvertSDFTo::Collection;

	UPROPERTY(EditAnywhere, Category = "Convert", meta = (UiMin = -1, UIMax = 1, EditCondition = "ConvertTo == EDataflowVolumeConvertSDFTo::Collection", EditConditionHides));
	float IsoValue = 0.f;

	UPROPERTY(EditAnywhere, Category = "Convert", meta = (ClampMin = 0, ClampMax = 1, EditCondition = "ConvertTo == EDataflowVolumeConvertSDFTo::Collection", EditConditionHides));
	float Adaptivity = 0.f;

	UPROPERTY(EditAnywhere, Category = "Convert", meta = (DisplayName = "Class", EditCondition = "ConvertTo == EDataflowVolumeConvertSDFTo::SDF", EditConditionHides));
	EDataflowVolumeConvertSDFGridClass GridClass = EDataflowVolumeConvertSDFGridClass::NoChange;

	UPROPERTY(EditAnywhere, Category = "Convert", meta = (DisplayName = "Type", EditCondition = "ConvertTo == EDataflowVolumeConvertSDFTo::SDF && GridClass == EDataflowVolumeConvertSDFGridClass::NoChange", EditConditionHides));
	EDataflowVolumeConvertSDFGridType GridType = EDataflowVolumeConvertSDFGridType::NoChange;

	UPROPERTY(EditAnywhere, Category = "Convert", meta = (ClampMin = 0, ClampMax = 1, EditCondition = "ConvertTo == EDataflowVolumeConvertSDFTo::SDF && GridClass == EDataflowVolumeConvertSDFGridClass::FogToSDF", EditConditionHides));
	float FogIsoValue = 0.5f;

	UPROPERTY(EditAnywhere, Category = "Convert", meta = (EditCondition = "ConvertTo == EDataflowVolumeConvertSDFTo::SDF", EditConditionHides));
	bool bPruneTolerance = true;

	UPROPERTY(EditAnywhere, Category = "Convert", meta = (EditCondition = "ConvertTo == EDataflowVolumeConvertSDFTo::SDF && bPruneTolerance == true", EditConditionHides));
	float Tolerance = 0.f;

	UPROPERTY(EditAnywhere, Category = "Convert", meta = (DisplayName = "Signed-Flood Fill Output", EditCondition = "ConvertTo == EDataflowVolumeConvertSDFTo::SDF", EditConditionHides));
	bool bFloodFillOutput = true;

	UPROPERTY(EditAnywhere, Category = "Convert", meta = (DisplayName = "Activate Interior Voxels", EditCondition = "ConvertTo == EDataflowVolumeConvertSDFTo::SDF", EditConditionHides));
	bool bActivateInteriror = true;

	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	UPROPERTY()
	bool bDrawVoxels = true;

	UPROPERTY()
	bool bDrawVoxelCenters = false;

	/** Collection output */
	UPROPERTY(meta = (DataflowOutput));
	FManagedArrayCollection Collection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FConvertVolumeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Uniformly scatter a total amount of points in active regions of a LevelSet or Fog volume
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FUniformVolumeScatterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUniformVolumeScatterDataflowNode, "UniformVolumeScatter", "Volume|Generators", "")

private:
	/** Float volume */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowFloatVolume FloatVolume;

	/** Minimum for the random range */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = 1));
	int32 MinNumberOfPoints = 100;

	/** Maximum for the random range */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = 1));
	int32 MaxNumberOfPoints = 100;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = "0"));
	int32 RandomSeed = 0;

	/** IsoValue */
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (DataflowInput));
	float IsoValue = 0.f;

	/** */
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (DataflowInput, ClampMin = 0, ClampMax = 1));
	float Spread = 1.f;

	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FLinearColor Color = FLinearColor(0.7f, 0.0f, 1.0f, 1.0f);

	/** Generated points */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FUniformVolumeScatterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Uniformly scatter a fixed number of points per active voxel of a LevelSet or Fog volume. If the pointsPerVoxel
 * value provided is a fractional value, each voxel calculates a delta value of
 * how likely it is to contain an extra point. 
 *
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FDenseUniformVolumeScatterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDenseUniformVolumeScatterDataflowNode, "DenseUniformVolumeScatter", "Volume|Generators", "")

private:
	/** Float volume */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowFloatVolume FloatVolume;

	/** Minimum for the random range */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = 0));
	float MinNumberOfPointsPerVoxel = 1.f;

	/** Maximum for the random range */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = 0));
	float MaxNumberOfPointsPerVoxel = 1.f;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = "0"));
	int32 RandomSeed = 0;

	/** IsoValue */
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (DataflowInput));
	float IsoValue = 0.f;

	/** */
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (DataflowInput, ClampMin = 0, ClampMax = 1));
	float Spread = 1.f;

	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FLinearColor Color = FLinearColor(0.7f, 0.0f, 1.0f, 1.0f);

	/** Generated points */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FDenseUniformVolumeScatterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Non uniformly scatter points per active voxel. The pointsPerVoxel value is used
 * to weight each grids cell value to compute a fixed number of points for every
 * active voxel. If the computed result is a fractional value, each voxel calculates
 * a delta value of how likely it is to contain an extra point.
 *
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FNonUniformVolumeScatterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FNonUniformVolumeScatterDataflowNode, "NonUniformVolumeScatter", "Volume|Generators", "")

private:
	/** Float volume */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowFloatVolume FloatVolume;

	/** Minimum for the random range */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = 0));
	float MinNumberOfPointsPerVoxel = .1f;

	/** Maximum for the random range */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = 0));
	float MaxNumberOfPointsPerVoxel = .3f;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = "0"));
	int32 RandomSeed = 0;

	/** IsoValue */
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (DataflowInput));
	float IsoValue = 0.f;

	/** */
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (DataflowInput, ClampMin = 0, ClampMax = 1));
	float Spread = 1.f;

	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FLinearColor Color = FLinearColor(0.7f, 0.0f, 1.0f, 1.0f);

	/** Generated points */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FNonUniformVolumeScatterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Sample a volume at input point's location
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FVolumeSampleToPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVolumeSampleToPointsDataflowNode, "VolumeSampleToPoints", "Volume|Sample", "")

private:
	/** Volume to sample */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowVolumeTypes Volume;

	/** Points to sample at */ 
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowVectorArrayTypes SamplePoints;

	UPROPERTY(EditAnywhere, Category = "Sampling");
	EDataflowVolumeSampleType SampleType = EDataflowVolumeSampleType::Float;

	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FLinearColor Color = FLinearColor(0.2f, 1.0f, 1.0f, 1.0f);

	UPROPERTY(EditAnywhere, Category = "Sampling")
	FString Attribute = FString("Attribute");

	// TArray<float>, TArray<int32>, TArray<FVector3f> 
	UPROPERTY(meta = (DataflowOutput));
	FDataflowAnyType OutArray;

	UPROPERTY(meta = (DataflowOutput));
	FManagedArrayCollection PointsCollection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FVolumeSampleToPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	void SetOutTypeFromSampleType();
	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) override;
};

/**
 * Extracts 2d slices from a volume
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FVolumeSliceDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVolumeSliceDataflowNode, "VolumeSlice", "Volume|Visualize", "Visualize Scalar Vector Curve 2D")

private:
	/** Volume to sample */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Volume", DataflowIntrinsic))
//	FDataflowFloatVolume Volume;
	FDataflowVolumeTypes Volume;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Slice Settings");
	EDataflowVolumeSliceMethod Method = EDataflowVolumeSliceMethod::Points;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Slice Settings");
	EDataflowVolumeSlicePlane Plane = EDataflowVolumeSlicePlane::XYPlane;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Slice Settings", meta = (ClampMin = "1", ClampMax = "10"));
	float BoundingBoxScale = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Slice Settings", meta = (ClampMin = "-1", ClampMax = "1"));
	float Offset = 0.f;

	UPROPERTY(EditAnywhere, Category = "Visualization Settings");
	bool bVisualize = true;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Visualization Settings - Scalar Visualization Settings", meta = (EditCondition = "bVisualize == true", EditConditionHides));
	EDataflowVolumeSliceRamp ColorRampType = EDataflowVolumeSliceRamp::InfraRed;

	UPROPERTY(EditAnywhere, Category = "Visualization Settings - Scalar Visualization Settings", meta = (EditCondition = "bVisualize == true", EditConditionHides, DisplayName = "ColorRamp"));
	FLinearColorRamp LinearColorRamp;

	UPROPERTY(EditAnywhere, Category = "Visualization Settings - Scalar Visualization Settings");
	bool bInvertColorRamp = false;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Visualization Settings - Scalar Visualization Settings", meta = (ClampMin = "-100", ClampMax = "100", EditCondition = "bVisualize == true", EditConditionHides));
	FVector2f Range = FVector2f(0.f, 1.f);

	UPROPERTY(EditAnywhere, Category = "Visualization Settings - Vector Visualization Settings", meta = (ClampMin = "0.01", ClampMax = "100", EditCondition = "bVisualize == true", EditConditionHides));
	float VectorScale = 1.f;

	UPROPERTY(EditAnywhere, Category = "Visualization Settings - Vector Visualization Settings", meta = (ClampMin = "0.01", ClampMax = "100", EditCondition = "bVisualize == true", EditConditionHides));
	float PointScale = 4.f;

	UPROPERTY(EditAnywhere, Category = "Visualization Settings - Vector Visualization Settings", meta = (ClampMin = "0.01", ClampMax = "100", EditCondition = "bVisualize == true", EditConditionHides));
	float LineWidth = 1.f;

	UPROPERTY(EditAnywhere, Category = "Visualization Settings - Vector Visualization Settings", meta = (ClampMin = "0.0001", ClampMax = "100", EditCondition = "bVisualize == true", EditConditionHides));
	float MinLength = 0.1f;

	UPROPERTY(EditAnywhere, Category = "Visualization Settings - Vector Visualization Settings", meta = (EditCondition = "bVisualize == true", EditConditionHides));
	FLinearColor PointColor = FLinearColor::Yellow;

	UPROPERTY(EditAnywhere, Category = "Visualization Settings - Vector Visualization Settings", meta = (EditCondition = "bVisualize == true", EditConditionHides));
	FLinearColor VectorColor = FLinearColor::Yellow;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) override;

#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif

	void OnColorCurveChanged(TArray<FRichCurve*> Curves);
public:
	FVolumeSliceDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * 
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FVolumeAnalysisDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVolumeAnalysisDataflowNode, "VolumeAnalysis", "Volume|Utilities", "")

private:
	/** Volume to process (scalar or vector volume) */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowVolumeTypes Volume;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Settings");
	EDataflowVolumeAnalysisOperator Operator = EDataflowVolumeAnalysisOperator::Gradient;

	/** Mask volume */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "MaskVolume"));
	FDataflowBoolVolume MaskVolume;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Settings");
	EDataflowVolumeAnalysisOutputName OutputName = EDataflowVolumeAnalysisOutputName::CustomName;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "OutputName == EDataflowVolumeAnalysisOutputName::CustomName", EditConditionHides));
	FString CustomName = FString("Custom");

	/** Volume output (scalar or vector volume) */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVolumeTypes OutVolume;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FVolumeAnalysisDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	void SetOutTypeFromOperator();
	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) override;
};

/**
 *
 *
 *
 */
USTRUCT(meta = (DataflowVolume))
struct FVolumeCombineDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVolumeCombineDataflowNode, "VolumeCombine", "Volume|Utilities", "")

private:
	/** Float A volume */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowFloatVolume FloatVolumeA;

	/** Float B volume */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowFloatVolume FloatVolumeB;

	UPROPERTY(EditAnywhere, Category = "Combine");
	EDataflowVolumeSDFCombineOperation Operation = EDataflowVolumeSDFCombineOperation::SDFUnion;

	UPROPERTY(EditAnywhere, Category = "Combine");
	bool bFlipInputs = false;

	UPROPERTY(EditAnywhere, Category = "Combine", meta = (DataflowInput, DisplayName = "A Multiplier", UIMin = -10.f, UIMax = 10.f));
	float MultiplierA = 1.f;

	UPROPERTY(EditAnywhere, Category = "Combine", meta = (DataflowInput, DisplayName = "B Multiplier", UIMin = -10.f, UIMax = 10.f));
	float MultiplierB = 1.f;

	UPROPERTY(EditAnywhere, Category = "Combine");
	EDataflowVolumeSDFCombineResample Resample = EDataflowVolumeSDFCombineResample::BMatchA;

	UPROPERTY(EditAnywhere, Category = "Combine");
	EDataflowVolumeSDFCombineInterpolation Interpolation = EDataflowVolumeSDFCombineInterpolation::Linear;

	UPROPERTY(EditAnywhere, Category = "Combine", meta = (DisplayName = "Prune Degenarate Tiles"));
	bool bPruneDegenerateTiles = false;

	UPROPERTY(EditAnywhere, Category = "Combine", meta = (DisplayName = "Prune Tolerance", UIMin = 0.f, UIMax = 1.f));
	float PruneTol = 0.f;

	// These properties are not needed anymore as we use the new rendering system. 
	// Since we don't wan't to version up the node we will just hide them.
	UPROPERTY()
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	UPROPERTY()
	bool bDrawVoxels = true;

	UPROPERTY()
	bool bDrawVoxelCenters = false;

	/** Output Float volume */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFloatVolume FloatVolume;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FVolumeCombineDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/*
* Terminal node to a save volume texture
*/
USTRUCT(meta = (DataflowTerminal))
struct FVolumeTextureTerminalNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVolumeTextureTerminalNode, "VolumeTextureTerminal", "Terminal", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowIntrinsic, DataflowPassthrough = "Volume"))
	FDataflowVolumeTypes Volume;

	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	TObjectPtr<UVolumeTexture> VolumeTextureAsset = nullptr;

	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;

public:
	FVolumeTextureTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * 
 *
 */
USTRUCT(meta = (Experimental, DataflowVolume))
struct FVolumeToSpheresWithRelaxationDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVolumeToSpheresWithRelaxationDataflowNode, "VolumeToSpheresWithRelaxation", "Volume|Utilities", "")

private:
	/** Float volume */
	UPROPERTY(meta = (DataflowInput))
	FDataflowFloatVolume FloatVolume;

	/** Number of points to scatter inside of the volume */
	UPROPERTY(EditAnywhere, Category = "PointScatter", meta = (DataflowInput, UIMin = 0, ClampMin = 0));
	int32 PointCount = 100;

	/** Random seed for scatter */
	UPROPERTY(EditAnywhere, Category = "PointScatter", meta = (DataflowInput, UIMin = 0, ClampMin = 0));
	int32 ScatterRandomSeed = 0;

	/** Spread for OpenVDB scatter */
	UPROPERTY(EditAnywhere, Category = "PointScatter", meta = (DataflowInput, UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1));
	float Spread = 1.f;

	/** Minimum radius for spheres */
	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, DisplayName = "Minimum Radius", UIMin = 0.001, ClampMin = 0.001f));
	float MinRadius = 1.f;

	/** Maximum radius for spheres */
	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, DisplayName = "Maximum Radius", UIMin = 0.001, ClampMin = 0.001f));
	float MaxRadius = 10.f;

	/** Random seed for radius */
	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput, UIMin = 0, ClampMin = 0));
	int32 RandomSeed = 0;

	/** Isovalue used for scattering */
	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (DataflowInput));
	float IsoValue = 0.f;

	/** Remove spheres which are completely inside of another sphere */
	UPROPERTY(EditAnywhere, Category = "Spheres");
	bool bRemoveEmbededSpheres = true;

	/** Apply relaxation to the remaining spheres after removal */
	UPROPERTY(EditAnywhere, Category = "Relaxation");
	bool bApplyRelaxation = true;

	/** Number of iterations for the relaxation */
	UPROPERTY(EditAnywhere, Category = "Relaxation", meta = (DataflowInput, DisplayName = "Iteration", UIMin = 1, ClampMin = 1, UIMax = 20, EditCondition = "bApplyRelaxation"));
	int32 Steps = 10;

	/** Scale the displacement applied to separate spheres */
	UPROPERTY(EditAnywhere, Category = "Relaxation", meta = (DataflowInput, UIMin = 0, ClampMin = 0.f, EditCondition = "bApplyRelaxation"));
	float StepScalar = 1.f;

	/** Move spheres so their distance from the isosurface equals to DistanceThreshold */
	UPROPERTY(EditAnywhere, Category = "Relaxation", meta = (DisplayName = "Correct against Isosurface", EditCondition = "bApplyRelaxation"));
	bool bCorrectAgainstSurface = true;
	
	/** Amount allowed of a sphere to be outside of the isosurface */
	UPROPERTY(EditAnywhere, Category = "Relaxation", meta = (DataflowInput, ClampMin = 0.f, EditCondition = "bApplyRelaxation && bCorrectAgainstSurface"));
	float DistanceThreshold = 0.f;

	/** Output spheres */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FSphere> Spheres;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FVolumeToSpheresWithRelaxationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void RegisterVolumeNodes();
}

