// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVolumeNodesPlugin.h"

#include "CoreMinimal.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowVolumeNodes.h"
#include "Dataflow/DataflowVolumeNodeAndPinTypeColors.h"
#include "Dataflow/DataflowVolumeAnyType.h"
#include "Dataflow/DataflowColorRampCustomization.h"
#include "Dataflow/DataflowVolumeRenderableType.h"

void IDataflowVolumetricPlugin::StartupModule()
{
	// anytypes need to be at the top as the connection registration in the nodes are using it
	UE::Dataflow::RegisterVolumeAnyTypes();

	UE::Dataflow::RegisterVolumeNodes();

	UE::Dataflow::RegisterVolumeColors();

	// Register Volume render type
	UE::Dataflow::Private::RegisterVolumeRenderableTypes();

	FModuleManager::Get().LoadModule("DataflowVolumeCore");
}

void IDataflowVolumetricPlugin::ShutdownModule()
{
}

IMPLEMENT_MODULE(IDataflowVolumetricPlugin, DataflowVolumeNodes)
