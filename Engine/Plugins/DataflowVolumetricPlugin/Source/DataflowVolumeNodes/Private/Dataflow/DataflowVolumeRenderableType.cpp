// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVolumeRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"
#include "Dataflow/DataflowEngineUtil.h"

#include "Components/BoxComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Drawing/PointSetComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"
#include "Dataflow/DataflowVolume.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"

#include "Materials/MaterialInstanceDynamic.h"

#include "UObject/ObjectPtr.h"

#include "Dataflow/DataflowFloatVolume.h"
#include "Dataflow/DataflowFloatVectorVolume.h"
#include "Dataflow/DataflowBoolVolume.h"
#include "Dataflow/DataflowIntVolume.h"

UDataflowVolumeRenderSettings::UDataflowVolumeRenderSettings()
{
	ColorSettings.bWireframe = false;
	ColorSettings.Transparency = 0.6f;
	ColorSettings.Color = FLinearColor(FVector(0.624f, 0.465f, 0.776f));
}

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FVolumeSurfaceRenderableType : public IRenderableType
	{
		virtual const FDataflowVolume& GetVolume(const FRenderableTypeInstance& Instance) const = 0;
		virtual const FName GetComponentName() const = 0;

		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Surface);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowVolumeRenderSettings);

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			UStaticMesh* BoxMesh = UE::Dataflow::RenderGeometry::GetDataflowBox();
			if (BoxMesh)
			{
				const FDataflowVolume& Volume = GetVolume(Instance);

				const UDataflowVolumeRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowVolumeRenderSettings>();
				if (Settings)
				{
					TArray<FBox> BoxArray;
					Volume.GetActiveVoxels(Settings->IsoValue, BoxArray, true);

					if (BoxArray.Num())
					{
						const FName BoxArrayComponentName = GetComponentName();
						const int32 NumBoxes = BoxArray.Num();

						UInstancedStaticMeshComponent* ISMComponent = OutComponents.AddNewComponent<UInstancedStaticMeshComponent>(BoxArrayComponentName);
						if (ISMComponent)
						{
							ISMComponent->PreAllocateInstancesMemory(NumBoxes);
							ISMComponent->SetStaticMesh(BoxMesh);

							ISMComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);

							FBox ComponentBoundingBox;
							for (const FBox& Box : BoxArray)
							{
								ComponentBoundingBox += Box;

								FTransform BoxTransform;
								BoxTransform.SetTranslation(Box.GetCenter());
								BoxTransform.SetScale3D(Settings->VoxelScale * Box.GetExtent() / 50.f);

								ISMComponent->AddInstance(BoxTransform);
							}

							ISMComponent->SetNumCustomDataFloats(3); // RGB

							UMaterialInterface* Material = Settings->ColorSettings.bWireframe
								? UE::Dataflow::RenderMaterial::GetDataflowColorWireframeMaterial()
								: UE::Dataflow::RenderMaterial::GetDataflowColorMaterial();
							if (Material)
							{
								UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(Material, ISMComponent);
								if (MaterialInstance)
								{
									ISMComponent->SetMaterial(0, MaterialInstance);

									MaterialInstance->SetScalarParameterValue(FName("Transparency"), Settings->ColorSettings.Transparency);

									TArray<float> Colors; // NumInstances * 3
									Colors.Init(1.0, BoxArray.Num() * 3);

									int32 Idx = 0;
									for (const FBox& Box : BoxArray)
									{
										FLinearColor Color = UE::Dataflow::Rendering::GetColor(Idx,
											ComponentBoundingBox,
											Box.GetCenter(),
											2.0 * Box.GetExtent().GetMax(),
											Settings->ColorSettings);

										Colors[3 * Idx + 0] = Color.R;
										Colors[3 * Idx + 1] = Color.G;
										Colors[3 * Idx + 2] = Color.B;

										Idx++;
									}

									ISMComponent->SetCustomData(0, BoxArray.Num() - 1, Colors);
								}
							}
						}
					}
				}
			}
		}
	};

	class FFloatVolumeSurfaceRenderableType : public FVolumeSurfaceRenderableType
	{
		virtual const FDataflowVolume& GetVolume(const FRenderableTypeInstance& Instance) const override
		{
			static FDataflowFloatVolume Default;

			return GetFloatVolume(Instance, Default);
		}

		virtual const FName GetComponentName() const override
		{
			static FName ComponentName = FName("FloatVolume");

			return ComponentName;
		}

		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FDataflowFloatVolume, FloatVolume);
	};

	class FIntVolumeSurfaceRenderableType : public FVolumeSurfaceRenderableType
	{
		virtual const FDataflowVolume& GetVolume(const FRenderableTypeInstance& Instance) const override
		{
			static FDataflowIntVolume Default;

			return GetIntVolume(Instance, Default);
		}

		virtual const FName GetComponentName() const override
		{
			static FName ComponentName = FName("IntVolume");

			return ComponentName;
		}

		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FDataflowIntVolume, IntVolume);
	};

	class FBoolVolumeSurfaceRenderableType : public FVolumeSurfaceRenderableType
	{
		virtual const FDataflowVolume& GetVolume(const FRenderableTypeInstance& Instance) const override
		{
			static FDataflowBoolVolume Default;

			return GetBoolVolume(Instance, Default);
		}

		virtual const FName GetComponentName() const override
		{
			static FName ComponentName = FName("BoolVolume");

			return ComponentName;
		}

		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FDataflowBoolVolume, BoolVolume);
	};

	class FFloatVectorVolumeSurfaceRenderableType : public FVolumeSurfaceRenderableType
	{
		virtual const FDataflowVolume& GetVolume(const FRenderableTypeInstance& Instance) const override
		{
			static FDataflowFloatVectorVolume Default;

			return GetFloatVectorVolume(Instance, Default);
		}

		virtual const FName GetComponentName() const override
		{
			static FName ComponentName = FName("FloatVectorVolume");

			return ComponentName;
		}

		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FDataflowFloatVectorVolume, FloatVectorVolume);
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterVolumeRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FFloatVolumeSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FIntVolumeSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FBoolVolumeSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FFloatVectorVolumeSurfaceRenderableType);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

}