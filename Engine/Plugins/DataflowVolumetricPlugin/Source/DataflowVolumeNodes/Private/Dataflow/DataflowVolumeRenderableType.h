// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "DataflowVolumeRenderableType.generated.h"

UCLASS(MinimalAPI, AutoExpandCategories = "Surface|ConstantColor, Surface|RandomColor, Surface|ColorBySize")
class UDataflowVolumeRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

public:
	UDataflowVolumeRenderSettings();

	UPROPERTY(EditAnywhere, Category = "Volume", meta = (ShowOnlyInnerProperties))
	FDataflowColorCommonRenderSettings ColorSettings;

	/** Voxels to display the volume gets generated at the Isovalue */
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (UIMin = -3, UIMax = 3))
	float IsoValue = 0.f;

	/** Scale applied to display the voxels */
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (UIMin = 0.001f, ClampMin = 0.001f, UIMax = 1.f, ClampMax = 1.f))
	float VoxelScale = .75f;
};

namespace UE::Dataflow::Private
{
	void RegisterVolumeRenderableTypes();
}
